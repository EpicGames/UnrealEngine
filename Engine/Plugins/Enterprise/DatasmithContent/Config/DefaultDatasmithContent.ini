[CoreRedirects]
+PackageRedirects=(OldName="/DatasmithContent/Materials/AliasMaster.AliasMaster",NewName="/DatasmithContent/Materials/AliasReference.AliasReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/C4DMaster.C4DMaster",NewName="/DatasmithContent/Materials/C4DReference.C4DReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/C4DMasterUV.C4DMasterUV",NewName="/DatasmithContent/Materials/C4DReferenceUV.C4DReferenceUV")
+PackageRedirects=(OldName="/DatasmithContent/Materials/CE_BaseMaster.CE_BaseMaster",NewName="/DatasmithContent/Materials/CE_BaseReference.CE_BaseReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/CE_OpacityMaster.CE_OpacityMaster",NewName="/DatasmithContent/Materials/CE_OpacityReference.CE_OpacityReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/CE_OpacitySimpleMaster.CE_OpacitySimpleMaster",NewName="/DatasmithContent/Materials/CE_OpacitySimpleReference.CE_OpacitySimpleReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/CE_OpaqueMaster.CE_OpaqueMaster",NewName="/DatasmithContent/Materials/CE_OpaqueReference.CE_OpaqueReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/IFC_ColorMaster.IFC_ColorMaster",NewName="/DatasmithContent/Materials/IFC_ColorReference.IFC_ColorReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/RevitMaster.RevitMaster",NewName="/DatasmithContent/Materials/RevitReference.RevitReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/SketchUpMaster.SketchUpMaster",NewName="/DatasmithContent/Materials/SketchUpReference.SketchUpReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/DeltaGenMaster.DeltaGenMaster",NewName="/DatasmithContent/Materials/FBXImporter/DeltaGenReference.DeltaGenReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/DeltaGenMasterAbsorb.DeltaGenMasterAbsorb",NewName="/DatasmithContent/Materials/FBXImporter/DeltaGenReferenceAbsorb.DeltaGenReferenceAbsorb")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/DeltaGenMasterClearCoat.DeltaGenMasterClearCoat",NewName="/DatasmithContent/Materials/FBXImporter/DeltaGenReferenceClearCoat.DeltaGenReferenceClearCoat")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/DeltaGenMasterTransparent.DeltaGenMasterTransparent",NewName="/DatasmithContent/Materials/FBXImporter/DeltaGenReferenceTransparent.DeltaGenReferenceTransparent")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/VREDMaster.VREDMaster",NewName="/DatasmithContent/Materials/FBXImporter/VREDReference.VREDReference")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/VREDMasterAbsorb.VREDMasterAbsorb",NewName="/DatasmithContent/Materials/FBXImporter/VREDReferenceAbsorb.VREDReferenceAbsorb")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/VREDMasterClearCoat.VREDMasterClearCoat",NewName="/DatasmithContent/Materials/FBXImporter/VREDReferenceClearCoat.VREDReferenceClearCoat")
+PackageRedirects=(OldName="/DatasmithContent/Materials/FBXImporter/VREDMasterTransparent.VREDMasterTransparent",NewName="/DatasmithContent/Materials/FBXImporter/VREDReferenceTransparent.VREDReferenceTransparent")

[SectionsToSave]
+Section=/Script/DatasmithContent.DatasmithCommonTessellationOptions
+Section=/Script/DatasmithContent.DatasmithImportOptions
