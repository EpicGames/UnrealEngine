// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DatasmithContent : ModuleRules
	{
		public DatasmithContent(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CinematicCamera",
					"Core",
					"CoreUObject",
					"Engine",
					"RenderCore",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Landscape",
					"LevelSequence",
					"MeshDescription",
					"Projects",
					"StaticMeshDescription",
					"VariantManagerContent",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"MainFrame",
				}
				);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"EditorFramework",
						"UnrealEd"
					}
				);
			}
		}
	}
}