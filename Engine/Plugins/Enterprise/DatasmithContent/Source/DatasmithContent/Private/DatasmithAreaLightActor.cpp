// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithAreaLightActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithAreaLightActor)

ADatasmithAreaLightActor::ADatasmithAreaLightActor()
	: Mobility( EComponentMobility::Static )
	, LightType( EDatasmithAreaLightActorType::Point )
	, LightShape( EDatasmithAreaLightActorShape::Rectangle )
	, Dimensions( 100.f, 100.f )
	, Intensity( 10.f )
	, IntensityUnits( ELightUnits::Candelas )
	, Color( FLinearColor::White )
	, Temperature( 6500.f )
	, IESTexture( nullptr )
	, Rotation( ForceInit )
	, SourceRadius( 0.f )
	, SourceLength( 0.f )
	, AttenuationRadius( 1000.f )
	, SpotlightInnerAngle(1.f)
	, SpotlightOuterAngle(44.f)
{
}
