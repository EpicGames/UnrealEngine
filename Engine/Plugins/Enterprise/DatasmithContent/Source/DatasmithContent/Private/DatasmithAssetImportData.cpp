// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithAssetImportData.h"

#include "DatasmithContentModule.h"
#include "DatasmithScene.h"

#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "EngineUtils.h"
#include "HAL/FileManager.h"
#include "Materials/MaterialInterface.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/EnterpriseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithAssetImportData)

#if WITH_EDITORONLY_DATA
void FDatasmithImportInfo::GetAssetRegistryTags(TArray<UObject::FAssetRegistryTag>& OutTags) const
{
	FAssetRegistryTagsContextData TagsContext(nullptr, EAssetRegistryTagsCaller::Uncategorized);
	GetAssetRegistryTags(TagsContext);
	for (TPair<FName, UObject::FAssetRegistryTag>& Pair : TagsContext.Tags)
	{
		OutTags.Add(MoveTemp(Pair.Value));
	}
}

void FDatasmithImportInfo::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	using FAssetRegistryTag = UObject::FAssetRegistryTag;

	Context.AddTag(FAssetRegistryTag(TEXT("SourceUri"), SourceUri, FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(TEXT("SourceHash"), SourceHash, FAssetRegistryTag::TT_Hidden));
}

#if WITH_EDITOR
bool UDatasmithSceneImportData::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FDatasmithImportBaseOptions, bIncludeAnimation))
	{
		return BaseOptions.CanIncludeAnimation();
	}

	return true;
}
#endif //WITH_EDITOR

UDatasmithStaticMeshImportData* UDatasmithStaticMeshImportData::GetImportDataForStaticMesh( UStaticMesh* StaticMesh, TOptional< DefaultOptionsPair > DefaultImportOptions )
{
	if( !StaticMesh )
	{
		return nullptr;
	}

	UDatasmithStaticMeshImportData* ImportData = nullptr;

	ImportData = Cast< UDatasmithStaticMeshImportData >( StaticMesh->GetAssetImportData());
	if ( !ImportData )
	{
		ImportData = NewObject< UDatasmithStaticMeshImportData >( StaticMesh );

		if ( DefaultImportOptions.IsSet() )
		{
			ImportData->ImportOptions = DefaultImportOptions.GetValue().Key;
			ImportData->AssetImportOptions = DefaultImportOptions.GetValue().Value;
		}

		// Try to preserve the source file path if possible
		if ( StaticMesh->GetAssetImportData() != nullptr )
		{
			ImportData->SourceData = StaticMesh->GetAssetImportData()->SourceData;
		}

		StaticMesh->SetAssetImportData(ImportData);
	}

	return ImportData;
}
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
UDatasmithStaticMeshCADImportData * UDatasmithStaticMeshCADImportData::GetCADImportDataForStaticMesh(UStaticMesh * StaticMesh, TOptional< DefaultOptionsTuple > DefaultImportOptions)
{
	check( StaticMesh );

	UDatasmithStaticMeshCADImportData* ImportData = Cast< UDatasmithStaticMeshCADImportData >( StaticMesh->GetAssetImportData());

	if ( !ImportData )
	{
		ImportData = NewObject< UDatasmithStaticMeshCADImportData >( StaticMesh );


		// Try to preserve the source file path if possible
		if ( StaticMesh->GetAssetImportData() != nullptr )
		{
			ImportData->SourceData = StaticMesh->GetAssetImportData()->SourceData;
		}

		StaticMesh->SetAssetImportData(ImportData);
	}

	if ( DefaultImportOptions.IsSet() )
	{
		DefaultOptionsTuple& Value = DefaultImportOptions.GetValue();

		ImportData->TessellationOptions = Value.Get<0>();
		ImportData->ImportOptions = Value.Get<1>();
		ImportData->AssetImportOptions = Value.Get<2>();
	}

	return ImportData;
}

void UDatasmithStaticMeshCADImportData::SetResourcePath(const FString& FilePath)
{
	check(FPaths::FileExists(FilePath));

	ResourceFilename = FPaths::GetCleanFilename(FilePath);

	static TArray<FString> AuxiliaryExtentions = {
		TEXT(".ext"),
	};

	AuxiliaryFilenames.Empty(AuxiliaryExtentions.Num());
	for (const FString& Ext : AuxiliaryExtentions)
	{
		FString AuxililaryFilePath = FilePath + Ext;
		if (FPaths::FileExists(AuxililaryFilePath))
		{
			AuxiliaryFilenames.Add(FPaths::GetCleanFilename(AuxililaryFilePath));
		}
	}

	// Set ResourcePath as absolute path because CoreTech expects an absolute path
	ResourcePath = FPaths::ConvertRelativePathToFull(FilePath);
}

const FString& UDatasmithStaticMeshCADImportData::GetResourcePath()
{
	check(!ResourcePath.IsEmpty());

	return ResourcePath;
}
#endif // WITH_EDITORONLY_DATA

void UDatasmithStaticMeshCADImportData::Serialize(FArchive& Ar)
{
	UE_LOGF(LogDatasmithContent, Log, "Do not use the UDatasmithStaticMeshCADImportData to store CAD data as the class is now deprecated.");
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);

	// Serialize/Deserialize stripping flag to control serialization of tessellation data
	bool bIsEditorDataIncluded = true;
	if (Ar.CustomVer(FEnterpriseObjectVersion::GUID) >= FEnterpriseObjectVersion::FixSerializationOfBulkAndExtraData)
	{
		FStripDataFlags StripFlags( Ar );
		bIsEditorDataIncluded = !StripFlags.IsEditorDataStripped();
	}

#if WITH_EDITORONLY_DATA
	// Do not bother loading or saving the rest if there is no source file or Editor's data are not included
	if (!bIsEditorDataIncluded || SourceData.SourceFiles.Num() == 0)
	{
		return;
	}

	TArray<uint8> ByteArray;
	Ar << ByteArray;
#endif // WITH_EDITORONLY_DATA
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Helper function required since UObject doesn't have an interface to access AssetImportData
UAssetImportData* Datasmith::GetAssetImportData(UObject* Asset)
{
#if WITH_EDITOR
	if (UStaticMesh* Mesh = Cast<UStaticMesh>(Asset))
	{
		return Mesh->GetAssetImportData();
	}
	else if (UTexture* Texture = Cast<UTexture>(Asset))
	{
		return Texture->AssetImportData;
	}
	else if (UMaterialInterface* Material = Cast<UMaterialInterface>(Asset))
	{
		return Material->AssetImportData;
	}
#endif // WITH_EDITOR

	return nullptr;
}

