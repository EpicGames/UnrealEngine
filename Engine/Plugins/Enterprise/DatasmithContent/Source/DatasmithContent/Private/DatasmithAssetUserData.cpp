// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithAssetUserData.h"

#include "DatasmithContentModule.h"

#include "GameFramework/Actor.h"
#include "Interfaces/Interface_AssetUserData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithAssetUserData)

#if WITH_EDITORONLY_DATA

bool UDatasmithAssetUserData::IsPostLoadThreadSafe() const
{
	return true;
}

void UDatasmithAssetUserData::PostLoad()
{
	Super::PostLoad();

	// RF_Transactional flag can cause a crash on save for Blueprint instances, and old data was flagged.
	ClearFlags(RF_Transactional);

	// A serialization issue caused nullptr to be serialized instead of valid UDatasmithObjectTemplate pointers.
	// This cleanup ensure values from this map can always be dereferenced
	for (auto It = ObjectTemplates.CreateIterator(); It; ++It)
	{
		if (!It->Value)
		{
			It.RemoveCurrent();
			UE_LOGF(LogDatasmithContent, Warning, "Serialization issue: null value found in templates")
		}
	}
}

#endif // WITH_EDITORONLY_DATA

const TCHAR* UDatasmithAssetUserData::UniqueIdMetaDataKey = TEXT("Datasmith_UniqueId");

UDatasmithAssetUserData* UDatasmithAssetUserData::GetDatasmithUserData(UObject* Object)
{
	if (AActor* Actor = Cast<AActor>(Object))
	{
		Object = Actor->GetRootComponent();
	}

	if (IInterface_AssetUserData* AssetUserData = Cast<IInterface_AssetUserData>(Object))
	{
		return Cast<UDatasmithAssetUserData>(AssetUserData->GetAssetUserDataOfClass(UDatasmithAssetUserData::StaticClass()));
	}

	return nullptr;
}

FString UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(UObject* Object, FName Key, bool bPartialMatchKey)
{
	if (Object)
	{
		if (UDatasmithAssetUserData* AssetUserData = GetDatasmithUserData(Object))
		{
			FString Value;

			if (bPartialMatchKey)
			{
				const FString KeyString = Key.ToString();

				for (TPair<FName, FString> KeyValuePair : AssetUserData->MetaData)
				{
					if (KeyValuePair.Key.ToString().Contains(KeyString))
					{
						Value = KeyValuePair.Value;
						break;
					}
				}
			}
			else
			{
				if (FString* ValuePtr = AssetUserData->MetaData.Find(Key))
				{
					Value = *ValuePtr;
				}
			}
			
			return Value;
		}
	}
	return FString();
}

TArray<FString> UDatasmithAssetUserData::GetDatasmithUserDataValuesForKey(UObject* Object, FName Key, bool bPartialMatchKey)
{
	if (Object)
	{
		if (UDatasmithAssetUserData* AssetUserData = GetDatasmithUserData(Object))
		{
			TArray<FString> Values;

			if (bPartialMatchKey)
			{
				const FString KeyString = Key.ToString();

				for (TPair<FName, FString> KeyValuePair : AssetUserData->MetaData)
				{
					if (KeyValuePair.Key.ToString().Contains(KeyString))
					{
						Values.Add(KeyValuePair.Value);
					}
				}
			}
			else
			{
				if (FString* ValuePtr = AssetUserData->MetaData.Find(Key))
				{
					Values.Add(*ValuePtr);
				}
			}

			return Values;
		}
	}
	return {};
}

bool UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(UObject* Object, FName Key, const FString & Value)
{
	// For AActor, the interface is actually implemented by the ActorComponent
	if (AActor* Actor = Cast<AActor>(Object))
	{
		Object = Actor->GetRootComponent();
	}

	if (IInterface_AssetUserData* AssetUserData = Cast<IInterface_AssetUserData>(Object))
	{
		UDatasmithAssetUserData* DatasmithUserData = AssetUserData->GetAssetUserData< UDatasmithAssetUserData >();

		if (!DatasmithUserData)
		{
			DatasmithUserData = NewObject<UDatasmithAssetUserData>(Object, NAME_None, RF_Public | RF_Transactional);
			AssetUserData->AddAssetUserData(DatasmithUserData);
		}

		// Add Datasmith meta data
		DatasmithUserData->MetaData.Add(Key, Value);
		DatasmithUserData->MetaData.KeySort(FNameLexicalLess());

		return true;
	}

	return false;
}
