// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithContentBlueprintLibrary.h"

#include "DatasmithAssetUserData.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithContentBlueprintLibrary)

UDatasmithAssetUserData* UDatasmithContentBlueprintLibrary::GetDatasmithUserData(UObject* Object)
{
	return UDatasmithAssetUserData::GetDatasmithUserData(Object);
}

FString UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValueForKey(UObject* Object, FName Key, bool bPartialMatchKey)
{
	return UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(Object, Key, bPartialMatchKey);
}

TArray<FString> UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValuesForKey(UObject* Object, FName Key, bool bPartialMatchKey)
{
	return UDatasmithAssetUserData::GetDatasmithUserDataValuesForKey(Object, Key, bPartialMatchKey);
}

void UDatasmithContentBlueprintLibrary::GetDatasmithUserDataKeysAndValuesForValue(UObject* Object, const FString& StringToMatch, TArray<FName>& OutKeys, TArray<FString>& OutValues)
{
	OutKeys.Reset();
	OutValues.Reset();

	if (Object)
	{
		if (UDatasmithAssetUserData* AssetUserData = GetDatasmithUserData(Object))
		{
			for (const TPair<FName, FString>& Kvp : AssetUserData->MetaData)
			{
				if (Kvp.Value.Contains(StringToMatch))
				{
					OutKeys.Add(Kvp.Key);
					OutValues.Add(Kvp.Value);
				}
			}
		}
	}
}

#if WITH_EDITOR

void UDatasmithContentBlueprintLibrary::GetAllDatasmithUserData(TSubclassOf<UObject> ObjectClass, TArray<UDatasmithAssetUserData*>& OutUserData)
{
	OutUserData.Reset();

	if (ObjectClass && ObjectClass->IsChildOf<AActor>())
	{
		ObjectClass = UActorComponent::StaticClass();
	}

	for (TObjectIterator<UDatasmithAssetUserData> It; It; ++It)
	{
		UDatasmithAssetUserData* UserData = *It;
		UObject* Object = UserData->GetOuter();
		if (ObjectClass == nullptr || Object->IsA(ObjectClass.Get()))
		{
			OutUserData.Add(UserData);
		}
	}
}

void UDatasmithContentBlueprintLibrary::GetAllObjectsAndValuesForKey(FName Key, TSubclassOf<UObject> ObjectClass, TArray<UObject*>& OutObjects, TArray<FString>& OutValues)
{
	OutObjects.Reset();
	OutValues.Reset();

	if (Key.IsNone())
	{
		return;
	}

	TArray<UDatasmithAssetUserData*> AssetUserDatas;
	GetAllDatasmithUserData(ObjectClass, AssetUserDatas);

	for (UDatasmithAssetUserData* AssetUserData : AssetUserDatas)
	{
		if (FString* Value = AssetUserData->MetaData.Find(Key))
		{
			UObject* Object = AssetUserData->GetOuter();
			if (ObjectClass->IsChildOf<AActor>())
			{
				if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
				{
					Object = ActorComponent->GetOwner();
				}
			}
			OutObjects.Add(Object);
			OutValues.Add(*Value);
		}
	}
}

#endif // WITH_EDITOR
