// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithContentModule.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogDatasmithContent)

/**
 * DatasmithContent module implementation (private)
 */
class FDatasmithContentModule : public IDatasmithContentModule
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
		if (!TempDir.IsEmpty() && FPaths::DirectoryExists(TempDir))
		{
			// Clean up all transient files created during the process
			IFileManager::Get().DeleteDirectory(*TempDir, false, true);
		}
	}

	const FString& GetTempDir() const override
	{
		// For 5.8, in case users are using this method, create the temporary folder only on demand.
		// Hence the const_cast.
		if (TempDir.IsEmpty())
		{
			// Create temporary directory which will be used by UDatasmithStaticMeshCADImportData to store transient data
			const_cast<FDatasmithContentModule*>(this)->TempDir = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithContentTemp"));
			IFileManager::Get().MakeDirectory(*TempDir);
		}
		return TempDir;
	}

private:
	FString TempDir;
};

IMPLEMENT_MODULE(FDatasmithContentModule, DatasmithContent);
