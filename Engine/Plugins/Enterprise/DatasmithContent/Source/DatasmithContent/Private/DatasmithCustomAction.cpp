// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCustomAction.h"

#include "Misc/Paths.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithCustomAction)

// #ueent_todo: custom actions, more than 2 states:
// Currently, an action is applicable or not applicable.
// Eg. Currently, Retessellate action is applicable when there is some nurbs additional data.
// This is OK to show/hide the action, but hiding an item prevents user from discovering features / learn to use them.
// We could improve the situation with an other state:
// - Irrelevant : eg. retessellate on materials
// - Relevant : eg. retessellate on mesh with no nurbs data
// - Applicable : eg. retessellate on mesh with nurbs data
FDatasmithCustomActionManager::FDatasmithCustomActionManager()
{
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (!It->HasAnyClassFlags(CLASS_Abstract) && It->IsChildOf(UDatasmithCustomActionBase::StaticClass()))
		{
			UDatasmithCustomActionBase* ActionCDO = It->GetDefaultObject<UDatasmithCustomActionBase>();
			RegisteredActions.Emplace(ActionCDO);
		}
	}
}

TArray<UDatasmithCustomActionBase*> FDatasmithCustomActionManager::GetApplicableActions(const TArray<FAssetData>& SelectedAssets)
{
	TArray<UDatasmithCustomActionBase*> ApplicableActions;

	for (const TStrongObjectPtr<UDatasmithCustomActionBase>& Action : RegisteredActions)
	{
		if (Action.IsValid() && Action->CanApplyOnAssets(SelectedAssets))
		{
			ApplicableActions.Add(Action.Get());
		}
	}

	return ApplicableActions;
}

TArray<UDatasmithCustomActionBase*> FDatasmithCustomActionManager::GetApplicableActions(const TArray<AActor*>& SelectedActors)
{
	TArray<UDatasmithCustomActionBase*> ApplicableActions;

	for (const TStrongObjectPtr<UDatasmithCustomActionBase>& Action : RegisteredActions)
	{
		if (Action.IsValid() && Action->CanApplyOnActors(SelectedActors))
		{
			ApplicableActions.Add(Action.Get());
		}
	}

	return ApplicableActions;
}

