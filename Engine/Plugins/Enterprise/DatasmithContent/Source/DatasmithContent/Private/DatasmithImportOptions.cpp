// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImportOptions.h"

#include "UObject/UnrealType.h"

#include "HAL/FileManager.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithImportOptions)

FDatasmithReimportOptions::FDatasmithReimportOptions()
	: bUpdateActors(true)
	, bRespawnDeletedActors(false)
{
}

FDatasmithImportBaseOptions::FDatasmithImportBaseOptions()
	: SceneHandling(EDatasmithImportScene::CurrentLevel)
	, bIncludeGeometry(true)
	, bIncludeMaterial(true)
	, bIncludeLight(true)
	, bIncludeCamera(true)
	, bIncludeAnimation(true)
{
}

UDatasmithImportOptions::UDatasmithImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SearchPackagePolicy(EDatasmithImportSearchPackagePolicy::Current)
	, MaterialConflictPolicy(EDatasmithImportAssetConflictPolicy::Replace)
	, TextureConflictPolicy(EDatasmithImportAssetConflictPolicy::Replace)
	, StaticMeshActorImportPolicy(EDatasmithImportActorPolicy::Update)
	, LightImportPolicy(EDatasmithImportActorPolicy::Update)
	, CameraImportPolicy(EDatasmithImportActorPolicy::Update)
	, OtherActorImportPolicy(EDatasmithImportActorPolicy::Update)
	, MaterialQuality(EDatasmithImportMaterialQuality::UseNoFresnelCurves)
	, bUseSameOptions(false)
{
}

void UDatasmithImportOptions::UpdateNotDisplayedConfig( bool bIsAReimport )
{
	EDatasmithImportActorPolicy DefaultImportActorPolicy = EDatasmithImportActorPolicy::Update;

	if ( bIsAReimport )
	{
		if ( !ReimportOptions.bUpdateActors )
		{
			BaseOptions.SceneHandling = EDatasmithImportScene::AssetsOnly;
		}
		else
		{
			BaseOptions.SceneHandling = EDatasmithImportScene::CurrentLevel;

			if ( ReimportOptions.bRespawnDeletedActors )
			{
				DefaultImportActorPolicy = EDatasmithImportActorPolicy::Full;
			}
		}
	}

	// Update enum properties based on boolean values
	if (BaseOptions.bIncludeGeometry == true)
	{
		StaticMeshActorImportPolicy = DefaultImportActorPolicy;
	}
	else
	{
		StaticMeshActorImportPolicy = EDatasmithImportActorPolicy::Ignore;
	}

	if (BaseOptions.bIncludeMaterial == true)
	{
		MaterialConflictPolicy = EDatasmithImportAssetConflictPolicy::Replace;
		TextureConflictPolicy = EDatasmithImportAssetConflictPolicy::Replace;
	}
	else
	{
		MaterialConflictPolicy = EDatasmithImportAssetConflictPolicy::Ignore;
		TextureConflictPolicy = EDatasmithImportAssetConflictPolicy::Ignore;
	}

	if (BaseOptions.bIncludeLight == true)
	{
		LightImportPolicy = DefaultImportActorPolicy;
	}
	else
	{
		LightImportPolicy = EDatasmithImportActorPolicy::Ignore;
	}

	if (BaseOptions.bIncludeCamera == true)
	{
		CameraImportPolicy = DefaultImportActorPolicy;
	}
	else
	{
		CameraImportPolicy = EDatasmithImportActorPolicy::Ignore;
	}

	OtherActorImportPolicy = DefaultImportActorPolicy;

	MaterialQuality = EDatasmithImportMaterialQuality::UseRealFresnelCurves;

	// For the time being, by default, search for existing components, Materials, etc, is done in the destination package
	SearchPackagePolicy = EDatasmithImportSearchPackagePolicy::Current;
}

#if WITH_EDITOR
bool UDatasmithImportOptions::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FDatasmithImportBaseOptions, bIncludeAnimation))
	{
		return BaseOptions.CanIncludeAnimation();
	}

	return true;
}
#endif //WITH_EDITOR

FDatasmithStaticMeshImportOptions::FDatasmithStaticMeshImportOptions()
	: MinLightmapResolution( EDatasmithImportLightmapMin::LIGHTMAP_64 )
	, MaxLightmapResolution( EDatasmithImportLightmapMax::LIGHTMAP_512 )
	, bGenerateLightmapUVs( true )
	, bRemoveDegenerates( true )
{
}

int32 FDatasmithStaticMeshImportOptions::ConvertLightmapEnumToValue( EDatasmithImportLightmapMin EnumValue )
{
	int32 MinLightmapSize = 0;

	switch ( EnumValue )
	{
	case EDatasmithImportLightmapMin::LIGHTMAP_16:
		MinLightmapSize = 16;
		break;
	case EDatasmithImportLightmapMin::LIGHTMAP_32:
		MinLightmapSize = 32;
		break;
	case EDatasmithImportLightmapMin::LIGHTMAP_64:
		MinLightmapSize = 64;
		break;
	case EDatasmithImportLightmapMin::LIGHTMAP_128:
		MinLightmapSize = 128;
		break;
	case EDatasmithImportLightmapMin::LIGHTMAP_256:
		MinLightmapSize = 256;
		break;
	case EDatasmithImportLightmapMin::LIGHTMAP_512:
		MinLightmapSize = 512;
		break;
	default:
		MinLightmapSize = 32;
		break;
	}

	return MinLightmapSize;
}

int32 FDatasmithStaticMeshImportOptions::ConvertLightmapEnumToValue( EDatasmithImportLightmapMax EnumValue )
{
	int32 MaxLightmapSize = 0;

	switch ( EnumValue )
	{
	case EDatasmithImportLightmapMax::LIGHTMAP_64:
		MaxLightmapSize = 64;
		break;
	case EDatasmithImportLightmapMax::LIGHTMAP_128:
		MaxLightmapSize = 128;
		break;
	case EDatasmithImportLightmapMax::LIGHTMAP_256:
		MaxLightmapSize = 256;
		break;
	case EDatasmithImportLightmapMax::LIGHTMAP_512:
		MaxLightmapSize = 512;
		break;
	case EDatasmithImportLightmapMax::LIGHTMAP_1024:
		MaxLightmapSize = 1024;
		break;
	case EDatasmithImportLightmapMax::LIGHTMAP_2048:
		MaxLightmapSize = 2048;
		break;
	case EDatasmithImportLightmapMax::LIGHTMAP_4096:
		MaxLightmapSize = 4096;
		break;
	default:
		MaxLightmapSize = 512;
		break;
}

	return MaxLightmapSize;
	}

void UDatasmithOptionsBase::Serialize(FStructuredArchive::FRecord Record)
{
	FGuardValue_Bitfield(Record.GetUnderlyingArchive().ArNoDelta, true);
	Super::Serialize(Record);
}
