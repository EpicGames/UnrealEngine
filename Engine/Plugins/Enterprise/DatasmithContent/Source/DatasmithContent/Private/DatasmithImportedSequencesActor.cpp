// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImportedSequencesActor.h"
#include "CoreMinimal.h"
#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithImportedSequencesActor)

ADatasmithImportedSequencesActor::ADatasmithImportedSequencesActor(const FObjectInitializer& Init)
	: Super(Init)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;
}

void ADatasmithImportedSequencesActor::PlayLevelSequence(ULevelSequence* SequenceToPlay)
{
    if(SequenceToPlay)
    {
        UE_LOGF(LogTemp, Log, "Play %ls", *SequenceToPlay->GetName());
    }
}
