// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithScene.h"
#include "DatasmithAssetImportData.h"

#include "Engine/AssetUserData.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "LevelSequence.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/EnterpriseObjectVersion.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithScene)

#if WITH_EDITORONLY_DATA

enum
{
	DATASMITHSCENEBULKDATA_VER_INITIAL = 1, // Version 0 means we didn't have any bulk data
	DATASMITHSCENEBULKDATA_VER_CURRENT = DATASMITHSCENEBULKDATA_VER_INITIAL
};

#endif // #if WITH_EDITORONLY_DATA

UDatasmithScene::UDatasmithScene()
{
#if WITH_EDITOR
	bPreWorldRenameCallbackRegistered = false;
#endif
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // For Clothes destruction: UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
UDatasmithScene::~UDatasmithScene()
{
#if WITH_EDITOR
	if (bPreWorldRenameCallbackRegistered)
	{
		FWorldDelegates::OnPreWorldRename.RemoveAll(this);
	}
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDatasmithScene::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UDatasmithScene::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));

		AssetImportData->DatasmithImportInfo.GetAssetRegistryTags(Context);
	}
#endif
}

void UDatasmithScene::RegisterPreWorldRenameCallback()
{
#if WITH_EDITOR
	if (!bPreWorldRenameCallbackRegistered)
	{
		bPreWorldRenameCallbackRegistered = true;
		FWorldDelegates::OnPreWorldRename.AddUObject(this, &UDatasmithScene::OnPreWorldRename);
	}
#endif
}

#if WITH_EDITOR
void UDatasmithScene::OnPreWorldRename(UWorld* World, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename)
{
	FString WorldPath = World->GetOutermost()->GetPathName();

	// Level sequences might need to have their bindings fixed if they were bound in a previously unnamed, unsaved world
	// So dirty the level sequences if the world was saved (renamed from Untitled to something else) for the first time
	if (WorldPath.StartsWith(TEXT("/Temp/Untitled")))
	{
		for (const TPair< FName, TSoftObjectPtr< ULevelSequence > >& NameLevelSequencePair : LevelSequences)
		{
			const TSoftObjectPtr< ULevelSequence >& LevelSequence = NameLevelSequencePair.Value;
			if (LevelSequence.IsValid())
			{
				LevelSequence->MarkPackageDirty();
			}
		}
	}
}
#endif

void UDatasmithScene::Serialize( FArchive& Archive )
{
#if WITH_EDITORONLY_DATA
	if ( Archive.IsSaving() && !IsTemplate() )
	{
		BulkDataVersion = DATASMITHSCENEBULKDATA_VER_CURRENT; // Update BulkDataVersion to current version
	}
#endif // #if WITH_EDITORONLY_DATA

	Super::Serialize( Archive );

	Archive.UsingCustomVersion(FEnterpriseObjectVersion::GUID);

	// Serialize/Deserialize stripping flag to control serialization of bulk data
	bool bIsEditorDataIncluded = true;
	if (Archive.CustomVer(FEnterpriseObjectVersion::GUID) >= FEnterpriseObjectVersion::FixSerializationOfBulkAndExtraData)
	{
		FStripDataFlags StripFlags( Archive );
		bIsEditorDataIncluded = !StripFlags.IsEditorDataStripped();
	}

#if WITH_EDITORONLY_DATA
	if ( bIsEditorDataIncluded && BulkDataVersion >= DATASMITHSCENEBULKDATA_VER_INITIAL )
	{
		DatasmithSceneBulkData.Serialize( Archive, this );
	}

	if (Archive.IsLoading() && Archive.CustomVer(FEnterpriseObjectVersion::GUID) < FEnterpriseObjectVersion::HasUDataprepRecipe)
	{
		check(AssetImportData);
		check(AssetImportData->StaticClass()->IsChildOf(UDatasmithSceneImportData::StaticClass()));

		UDatasmithSceneImportData* SceneImportData = Cast<UDatasmithSceneImportData>(AssetImportData);
	}
#endif // #if WITH_EDITORONLY_DATA
}

void UDatasmithScene::AddAssetUserData( UAssetUserData* InUserData )
{
#if WITH_EDITORONLY_DATA
	if ( InUserData != NULL )
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass( InUserData->GetClass() );
		if ( ExistingData != NULL )
		{
			AssetUserData.Remove( ExistingData );
		}
		AssetUserData.Add( InUserData );
	}
#endif // #if WITH_EDITORONLY_DATA
}

UAssetUserData* UDatasmithScene::GetAssetUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass )
{
#if WITH_EDITORONLY_DATA
	for ( int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++ )
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if ( Datum != NULL && Datum->IsA( InUserDataClass ) )
		{
			return Datum;
		}
	}
#endif // #if WITH_EDITORONLY_DATA
	return NULL;

}

void UDatasmithScene::RemoveUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass )
{
#if WITH_EDITORONLY_DATA
	for ( int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++ )
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if ( Datum != NULL && Datum->IsA(InUserDataClass ) )
		{
			AssetUserData.RemoveAt( DataIdx );
			return;
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}

const TArray<UAssetUserData*>* UDatasmithScene::GetAssetUserDataArray() const
{
#if WITH_EDITORONLY_DATA
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#else
	return NULL;
#endif // #if WITH_EDITORONLY_DATA
}
