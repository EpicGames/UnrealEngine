// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneActor.h"
#include "DatasmithAssetUserData.h"
#include "DatasmithContentModule.h"

#include "Engine/Engine.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithSceneActor)

ADatasmithSceneActor::ADatasmithSceneActor()
{
#if WITH_EDITOR
	if (!IsTemplate())
	{
		if (HasAnyFlags(RF_NeedPostLoad) || GetOuter()->HasAnyFlags(RF_NeedPostLoad))
		{
			// Delegate registration is not thread-safe, so we postpone it on PostLoad when coming from loading which could be on another thread
		}
		else
		{
			RegisterDelegates();
		}
	}
#endif // WITH_EDITOR
}

void ADatasmithSceneActor::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	if (!IsTemplate())
	{
		UnregisterDelegates();
	}
#endif
}

#if WITH_EDITOR

void ADatasmithSceneActor::RegisterDelegates()
{
	ensureMsgf(IsInGameThread(), TEXT("Potential race condition in ADatasmithSceneActor registering to delegates from non game-thread"));

	FEditorDelegates::MapChange.AddUObject(this, &ADatasmithSceneActor::OnMapChange);

	if (GEngine)
	{
		OnActorDeletedDelegateHandle = GEngine->OnLevelActorDeleted().AddUObject(this, &ADatasmithSceneActor::OnActorDeleted);
	}

	if (GEditor)
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddUObject(this, &ADatasmithSceneActor::OnAssetPostImport);
	}
}

void ADatasmithSceneActor::UnregisterDelegates()
{
	ensureMsgf(IsInGameThread(), TEXT("Potential race condition in ADatasmithSceneActor unregistering to delegates from non game-thread"));

	FEditorDelegates::MapChange.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().Remove(OnActorDeletedDelegateHandle);
	}

	if (UImportSubsystem* ImportSubsystem = (GEditor ? GEditor->GetEditorSubsystem<UImportSubsystem>() : nullptr))
	{
		ImportSubsystem->OnAssetPostImport.RemoveAll(this);
	}
}

#endif

#if WITH_EDITOR

void ADatasmithSceneActor::PostLoad()
{
	if (!IsTemplate())
	{
		RegisterDelegates();
	}

	EnsureDatasmithIdsForRelatedActors();

	Super::PostLoad();
}

// Since we now rely on all datasmith related object to have a UniqueId, this will take care of fixing all related actors
void ADatasmithSceneActor::EnsureDatasmithIdsForRelatedActors()
{
	for (TPair< FName, TSoftObjectPtr< AActor > >& ActorPair : RelatedActors)
	{
		if (AActor* Actor = ActorPair.Value.Get())
		{
			FString DatasmithUniqueId = UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(Actor, UDatasmithAssetUserData::UniqueIdMetaDataKey);
			if (DatasmithUniqueId.IsEmpty())
			{
				if (!UDatasmithAssetUserData::SetDatasmithUserDataValueForKey(Actor, UDatasmithAssetUserData::UniqueIdMetaDataKey, ActorPair.Key.ToString()))
				{
					UE_LOGF(LogDatasmithContent, Warning, "Actor %ls is referenced by datasmith scene but doesn't have a datasmith uniqueId.", *Actor->GetName());
				}
			}
		}
	}
}

void ADatasmithSceneActor::OnMapChange(uint32 MapEventFlags)
{
	// Whenever a sub-level is loaded, we need to apply the fix
	if (MapEventFlags == MapChangeEventFlags::NewMap)
	{
		EnsureDatasmithIdsForRelatedActors();
	}
}

// Reset the related actor on the scene when an actor is deleted to avoid re-matching it with a manually created object that might have the same name.
void ADatasmithSceneActor::OnActorDeleted(AActor* ActorDeleted)
{
	FString DatasmithUniqueId = UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(ActorDeleted, UDatasmithAssetUserData::UniqueIdMetaDataKey);
	if (!DatasmithUniqueId.IsEmpty())
	{
		TSoftObjectPtr< AActor >* RelatedActorPtr = RelatedActors.Find(FName(*DatasmithUniqueId));
		if (RelatedActorPtr)
		{
			AActor* RelatedActor = RelatedActorPtr->Get();
			if (RelatedActor == ActorDeleted)
			{
				// this will add this actor to the transaction if there is one currently recording
				Modify();

				RelatedActorPtr->Reset();
			}
		}
	}
}

// Reattach related actors on cut/paste by matching their Datasmith UniqueId
void ADatasmithSceneActor::OnAssetPostImport(UFactory* InFactory, UObject* ActorAdded)
{
	for (TObjectIterator<AActor> It; It; ++It)
	{
		AActor* Actor = *It;

		FString DatasmithUniqueId = UDatasmithAssetUserData::GetDatasmithUserDataValueForKey(Actor, UDatasmithAssetUserData::UniqueIdMetaDataKey);
		if (!DatasmithUniqueId.IsEmpty())
		{
			TSoftObjectPtr< AActor >* RelatedActorPtr = RelatedActors.Find(FName(*DatasmithUniqueId));
			if (RelatedActorPtr)
			{
				AActor* RelatedActor = RelatedActorPtr->Get();
				if (RelatedActor == nullptr)
				{
					// this will add this actor to the transaction if there is one currently recording
					Modify();

					*RelatedActorPtr = Actor;
				}
			}
		}
	}
}

#endif // WITH_EDITOR
