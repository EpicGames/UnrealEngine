// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithActorTemplate.h"

#include "DatasmithAssetUserData.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithActorTemplate)

UObject* UDatasmithActorTemplate::UpdateObject(UObject* Destination, bool bForce)
{
	const USceneComponent* SceneComponent = Cast<USceneComponent>(Destination);
	AActor* ImportedActor = SceneComponent ? SceneComponent->GetOwner() : Cast<AActor>(Destination);
	if (!ImportedActor)
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithActorTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate<UDatasmithActorTemplate>(ImportedActor) : nullptr;

	if (!PreviousTemplate)
	{
		ImportedActor->Layers = Layers.Array();
		ImportedActor->Tags = Tags.Array();
	}
	else
	{
		ImportedActor->Layers = FDatasmithObjectTemplateUtils::ThreeWaySetMerge(PreviousTemplate->Layers, TSet<FName>(ImportedActor->Layers), Layers).Array();
		ImportedActor->Tags = FDatasmithObjectTemplateUtils::ThreeWaySetMerge(PreviousTemplate->Tags, TSet<FName>(ImportedActor->Tags), Tags).Array();
	}
#endif // #if WITH_EDITORONLY_DATA

	return ImportedActor;
}

void UDatasmithActorTemplate::Load(const UObject* Source)
{
#if WITH_EDITORONLY_DATA
	const USceneComponent* SceneComponent = Cast<USceneComponent>(Source);
	const AActor* SourceActor = SceneComponent ? SceneComponent->GetOwner() : Cast<AActor>(Source);
	if (!SourceActor)
	{
		return;
	}

	Layers = TSet<FName>(SourceActor->Layers);
	Tags = TSet<FName>(SourceActor->Tags);
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithActorTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithActorTemplate* TypedOther = Cast< UDatasmithActorTemplate >( Other );

	return TypedOther != nullptr
		&& FDatasmithObjectTemplateUtils::SetsEquals(TypedOther->Layers, Layers)
	    && FDatasmithObjectTemplateUtils::SetsEquals(TypedOther->Tags, Tags);
}
