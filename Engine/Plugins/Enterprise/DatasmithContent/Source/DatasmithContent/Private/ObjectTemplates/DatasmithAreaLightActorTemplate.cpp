// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithAreaLightActorTemplate.h"

#include "Engine/TextureLightProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithAreaLightActorTemplate)

UDatasmithAreaLightActorTemplate::UDatasmithAreaLightActorTemplate()
	: UDatasmithObjectTemplate(true)
{
	UDatasmithAreaLightActorTemplate::Load( ADatasmithAreaLightActor::StaticClass()->GetDefaultObject() );
}

UObject* UDatasmithAreaLightActorTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	const USceneComponent* SceneComponent = Cast< USceneComponent >( Destination );
	ADatasmithAreaLightActor* AreaLightActor = Cast< ADatasmithAreaLightActor >( SceneComponent ? SceneComponent->GetOwner() : Destination );

	if ( !AreaLightActor )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithAreaLightActorTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithAreaLightActorTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( LightType, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( LightShape, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Dimensions, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Color, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Intensity, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( IntensityUnits, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Temperature, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSETSOFTOBJECTPTR( IESTexture, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUseIESBrightness, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( IESBrightnessScale, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Rotation, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SourceRadius, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SourceLength, AreaLightActor, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( AttenuationRadius, AreaLightActor, PreviousTemplate );
#endif // #if WITH_EDITORONLY_DATA

	return AreaLightActor->GetRootComponent();
}

void UDatasmithAreaLightActorTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const ADatasmithAreaLightActor* AreaLightActor = Cast< ADatasmithAreaLightActor >( Source );

	if ( !AreaLightActor )
	{
		const UActorComponent* ActorComponent = Cast< UActorComponent >( Source );

		if ( !ActorComponent )
		{
			return;
		}
		else
		{
			AreaLightActor = Cast< ADatasmithAreaLightActor >( ActorComponent->GetOwner() );

			if ( !AreaLightActor )
			{
				return;
			}
		}
	}

	LightType = AreaLightActor->LightType;
	LightShape = AreaLightActor->LightShape;
	Dimensions = AreaLightActor->Dimensions;
	Color = AreaLightActor->Color;
	Intensity = AreaLightActor->Intensity;
	IntensityUnits = AreaLightActor->IntensityUnits;
	Temperature = AreaLightActor->Temperature;
	IESTexture = AreaLightActor->IESTexture;
	bUseIESBrightness = AreaLightActor->bUseIESBrightness;
	IESBrightnessScale = AreaLightActor->IESBrightnessScale;
	Rotation = AreaLightActor->Rotation;
	SourceRadius = AreaLightActor->SourceRadius;
	SourceLength = AreaLightActor->SourceLength;
	AttenuationRadius = AreaLightActor->AttenuationRadius;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithAreaLightActorTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithAreaLightActorTemplate* TypedOther = Cast< UDatasmithAreaLightActorTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = ( LightType == TypedOther->LightType );
	bEquals = bEquals && ( LightShape == TypedOther->LightShape );
	bEquals = bEquals && Dimensions.Equals( TypedOther->Dimensions );
	bEquals = bEquals && Color.Equals( TypedOther->Color );
	bEquals = bEquals && FMath::IsNearlyEqual( Intensity, TypedOther->Intensity );
	bEquals = bEquals && ( IntensityUnits == TypedOther->IntensityUnits );
	bEquals = bEquals && FMath::IsNearlyEqual( Temperature, TypedOther->Temperature );
	bEquals = bEquals && ( IESTexture == TypedOther->IESTexture );
	bEquals = bEquals && ( bUseIESBrightness == TypedOther->bUseIESBrightness );
	bEquals = bEquals && FMath::IsNearlyEqual( IESBrightnessScale, TypedOther->IESBrightnessScale );
	bEquals = bEquals && Rotation.Equals( TypedOther->Rotation );
	bEquals = bEquals && FMath::IsNearlyEqual( SourceRadius, TypedOther->SourceRadius );
	bEquals = bEquals && FMath::IsNearlyEqual( SourceLength, TypedOther->SourceLength );
	bEquals = bEquals && FMath::IsNearlyEqual( AttenuationRadius, TypedOther->AttenuationRadius );

	return bEquals;
}
