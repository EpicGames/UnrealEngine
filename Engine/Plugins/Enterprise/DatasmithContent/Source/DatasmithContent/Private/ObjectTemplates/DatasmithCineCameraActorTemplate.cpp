// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithCineCameraActorTemplate.h"

#include "ObjectTemplates/DatasmithActorTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithCineCameraActorTemplate)

FDatasmithCameraLookatTrackingSettingsTemplate::FDatasmithCameraLookatTrackingSettingsTemplate()
	: bEnableLookAtTracking( 0 )
	, bAllowRoll( 0 )
{
}

void FDatasmithCameraLookatTrackingSettingsTemplate::Apply( FCameraLookatTrackingSettings* Destination, FDatasmithCameraLookatTrackingSettingsTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bEnableLookAtTracking, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bAllowRoll, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSETSOFTOBJECTPTR( ActorToTrack, Destination, PreviousTemplate );
}

void FDatasmithCameraLookatTrackingSettingsTemplate::Load( const FCameraLookatTrackingSettings& Source )
{
	bEnableLookAtTracking = Source.bEnableLookAtTracking;
	bAllowRoll = Source.bAllowRoll;
	ActorToTrack = Source.ActorToTrack;
}

bool FDatasmithCameraLookatTrackingSettingsTemplate::Equals( const FDatasmithCameraLookatTrackingSettingsTemplate& Other ) const
{
	bool bEquals = ( bEnableLookAtTracking == Other.bEnableLookAtTracking );
	bEquals = bEquals && ( bAllowRoll == Other.bAllowRoll );
	bEquals = bEquals && ( ActorToTrack == Other.ActorToTrack );

	return bEquals;
}

UObject* UDatasmithCineCameraActorTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	ACineCameraActor* CineCameraActor = UDatasmithActorTemplate::GetActor< ACineCameraActor >( Destination );

	if ( !CineCameraActor )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithCineCameraActorTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithCineCameraActorTemplate >( Destination ) : nullptr;

	LookatTrackingSettings.Apply( &CineCameraActor->LookatTrackingSettings, PreviousTemplate ? &PreviousTemplate->LookatTrackingSettings : nullptr );
#endif // #if WITH_EDITORONLY_DATA

	return CineCameraActor->GetRootComponent();
}

void UDatasmithCineCameraActorTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const ACineCameraActor* CineCameraActor = Cast< ACineCameraActor >( Source );

	if ( !CineCameraActor )
	{
		return;
	}

	LookatTrackingSettings.Load( CineCameraActor->LookatTrackingSettings );
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithCineCameraActorTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithCineCameraActorTemplate* TypedOther = Cast< UDatasmithCineCameraActorTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = LookatTrackingSettings.Equals( TypedOther->LookatTrackingSettings );

	return bEquals;
}
