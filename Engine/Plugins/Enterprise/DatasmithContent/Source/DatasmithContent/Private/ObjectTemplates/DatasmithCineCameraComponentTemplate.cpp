// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithCineCameraComponentTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithCineCameraComponentTemplate)

void FDatasmithCameraFilmbackSettingsTemplate::Apply( FCameraFilmbackSettings* Destination, const FDatasmithCameraFilmbackSettingsTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SensorWidth, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SensorHeight, Destination, PreviousTemplate );
}

void FDatasmithCameraFilmbackSettingsTemplate::Load( const FCameraFilmbackSettings& Source )
{
	SensorWidth = Source.SensorWidth;
	SensorHeight = Source.SensorHeight;
}

bool FDatasmithCameraFilmbackSettingsTemplate::Equals( const FDatasmithCameraFilmbackSettingsTemplate& Other ) const
{
	bool bEquals = FMath::IsNearlyEqual( SensorWidth, Other.SensorWidth);
	bEquals = bEquals && FMath::IsNearlyEqual( SensorHeight, Other.SensorHeight);

	return bEquals;
}

FDatasmithPostProcessSettingsTemplate::FDatasmithPostProcessSettingsTemplate()
{
	Load( FPostProcessSettings() );
}

void FDatasmithPostProcessSettingsTemplate::Apply( FPostProcessSettings* Destination, const FDatasmithPostProcessSettingsTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_WhiteTemp, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( WhiteTemp, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_VignetteIntensity, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( VignetteIntensity, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_ColorSaturation, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( ColorSaturation, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_AutoExposureMethod, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( AutoExposureMethod, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_CameraISO, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( CameraISO, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_CameraShutterSpeed, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( CameraShutterSpeed, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bOverride_DepthOfFieldFstop, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( DepthOfFieldFstop, Destination, PreviousTemplate );
}

void FDatasmithPostProcessSettingsTemplate::Load( const FPostProcessSettings& Source )
{
	bOverride_WhiteTemp = Source.bOverride_WhiteTemp;
	WhiteTemp = Source.WhiteTemp;

	bOverride_VignetteIntensity = Source.bOverride_VignetteIntensity;
	VignetteIntensity = Source.VignetteIntensity;

	bOverride_ColorSaturation = Source.bOverride_ColorSaturation;
	ColorSaturation = Source.ColorSaturation;

	bOverride_AutoExposureMethod = Source.bOverride_AutoExposureMethod;
	AutoExposureMethod = Source.AutoExposureMethod;

	bOverride_CameraISO = Source.bOverride_CameraISO;
	CameraISO = Source.CameraISO;

	bOverride_CameraShutterSpeed = Source.bOverride_CameraShutterSpeed;
	CameraShutterSpeed = Source.CameraShutterSpeed;

	bOverride_DepthOfFieldFstop = Source.bOverride_DepthOfFieldFstop;
	DepthOfFieldFstop = Source.DepthOfFieldFstop;
}

bool FDatasmithPostProcessSettingsTemplate::Equals( const FDatasmithPostProcessSettingsTemplate& Other ) const
{
	bool bEquals = bOverride_WhiteTemp == Other.bOverride_WhiteTemp;
	bEquals = bEquals && FMath::IsNearlyEqual( WhiteTemp, Other.WhiteTemp );

	bEquals = bEquals && bOverride_VignetteIntensity == Other.bOverride_VignetteIntensity;
	bEquals = bEquals && FMath::IsNearlyEqual( VignetteIntensity, Other.VignetteIntensity );

	bEquals = bEquals && bOverride_ColorSaturation == Other.bOverride_ColorSaturation;
	bEquals = bEquals && ColorSaturation.Equals( Other.ColorSaturation );

	bEquals = bEquals && bOverride_AutoExposureMethod == Other.bOverride_AutoExposureMethod;
	bEquals = bEquals && AutoExposureMethod == Other.AutoExposureMethod;

	bEquals = bEquals && bOverride_CameraISO == Other.bOverride_CameraISO;
	bEquals = bEquals && FMath::IsNearlyEqual( CameraISO, Other.CameraISO );

	bEquals = bEquals && bOverride_CameraShutterSpeed == Other.bOverride_CameraShutterSpeed;
	bEquals = bEquals && FMath::IsNearlyEqual( CameraShutterSpeed, Other.CameraShutterSpeed );

	bEquals = bEquals && bOverride_DepthOfFieldFstop == Other.bOverride_DepthOfFieldFstop;
	bEquals = bEquals && FMath::IsNearlyEqual( DepthOfFieldFstop, Other.DepthOfFieldFstop );

	return bEquals;
}

void FDatasmithCameraLensSettingsTemplate::Apply( FCameraLensSettings* Destination, const FDatasmithCameraLensSettingsTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( MaxFStop, Destination, PreviousTemplate );
}

void FDatasmithCameraLensSettingsTemplate::Load( const FCameraLensSettings& Source )
{
	MaxFStop = Source.MaxFStop;
}

bool FDatasmithCameraLensSettingsTemplate::Equals( const FDatasmithCameraLensSettingsTemplate& Other ) const
{
	bool bEquals = FMath::IsNearlyEqual( MaxFStop, Other.MaxFStop);

	return bEquals;
}

void FDatasmithCameraFocusSettingsTemplate::Apply( FCameraFocusSettings* Destination, const FDatasmithCameraFocusSettingsTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( FocusMethod, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( ManualFocusDistance, Destination, PreviousTemplate );
}

void FDatasmithCameraFocusSettingsTemplate::Load( const FCameraFocusSettings& Source )
{
	FocusMethod = Source.FocusMethod;
	ManualFocusDistance = Source.ManualFocusDistance;
}

bool FDatasmithCameraFocusSettingsTemplate::Equals( const FDatasmithCameraFocusSettingsTemplate& Other ) const
{
	bool bEquals = ( FocusMethod == Other.FocusMethod );
	bEquals = bEquals && FMath::IsNearlyEqual( ManualFocusDistance, Other.ManualFocusDistance);

	return bEquals;
}

UObject* UDatasmithCineCameraComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	UCineCameraComponent* CineCameraComponent = Cast< UCineCameraComponent >( Destination );

	if ( !CineCameraComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithCineCameraComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithCineCameraComponentTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( CurrentFocalLength, CineCameraComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( CurrentAperture, CineCameraComponent, PreviousTemplate );

	FilmbackSettings.Apply( &CineCameraComponent->Filmback, PreviousTemplate ? &PreviousTemplate->FilmbackSettings : nullptr );
	LensSettings.Apply( &CineCameraComponent->LensSettings, PreviousTemplate ? &PreviousTemplate->LensSettings : nullptr );
	FocusSettings.Apply( &CineCameraComponent->FocusSettings, PreviousTemplate ? &PreviousTemplate->FocusSettings : nullptr );

	PostProcessSettings.Apply( &CineCameraComponent->PostProcessSettings, PreviousTemplate ? &PreviousTemplate->PostProcessSettings : nullptr );
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithCineCameraComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UCineCameraComponent* CineCameraComponent = Cast< UCineCameraComponent >( Source );

	if ( !CineCameraComponent )
	{
		return;
	}

	CurrentFocalLength = CineCameraComponent->CurrentFocalLength;
	CurrentAperture = CineCameraComponent->CurrentAperture;

	FilmbackSettings.Load( CineCameraComponent->Filmback );
	LensSettings.Load( CineCameraComponent->LensSettings );
	FocusSettings.Load( CineCameraComponent->FocusSettings );

	PostProcessSettings.Load( CineCameraComponent->PostProcessSettings );
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithCineCameraComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithCineCameraComponentTemplate* TypedOther = Cast< UDatasmithCineCameraComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = FMath::IsNearlyEqual( CurrentFocalLength, TypedOther->CurrentFocalLength );
	bEquals = bEquals && FMath::IsNearlyEqual( CurrentAperture, TypedOther->CurrentAperture );
	bEquals = bEquals && FilmbackSettings.Equals( TypedOther->FilmbackSettings );
	bEquals = bEquals && LensSettings.Equals( TypedOther->LensSettings );
	bEquals = bEquals && FocusSettings.Equals( TypedOther->FocusSettings );
	bEquals = bEquals && PostProcessSettings.Equals( TypedOther->PostProcessSettings );

	return bEquals;
}
