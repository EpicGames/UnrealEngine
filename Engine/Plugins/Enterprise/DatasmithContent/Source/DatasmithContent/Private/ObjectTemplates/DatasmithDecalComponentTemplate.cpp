// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithDecalComponentTemplate.h"

#include "Components/DecalComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithDecalComponentTemplate)

UDatasmithDecalComponentTemplate::UDatasmithDecalComponentTemplate()
{
}

UObject* UDatasmithDecalComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	UDecalComponent* DecalComponent = Cast< UDecalComponent >( Destination );

	if ( !DecalComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithDecalComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithDecalComponentTemplate >( Destination ) : nullptr;

	if ( !PreviousTemplate || DecalComponent->GetDecalMaterial() == PreviousTemplate->Material )
	{
		DecalComponent->SetDecalMaterial( Material );
	}

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SortOrder, DecalComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( DecalSize, DecalComponent, PreviousTemplate );
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithDecalComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UDecalComponent* DecalComponent = Cast< UDecalComponent >( Source );

	if ( !DecalComponent )
	{
		return;
	}

	SortOrder = DecalComponent->SortOrder;
	DecalSize = DecalComponent->DecalSize;
	Material = DecalComponent->GetDecalMaterial();
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithDecalComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithDecalComponentTemplate* TypedOther = Cast< UDatasmithDecalComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = SortOrder == TypedOther->SortOrder;
	bEquals = bEquals && ( DecalSize == TypedOther->DecalSize );
	bEquals = bEquals && ( Material == TypedOther->Material );

	return bEquals;
}
