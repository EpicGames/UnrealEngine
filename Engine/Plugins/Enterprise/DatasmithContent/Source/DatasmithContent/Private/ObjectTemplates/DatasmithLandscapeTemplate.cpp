// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithLandscapeTemplate.h"

#include "Landscape.h"
#include "ObjectTemplates/DatasmithActorTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithLandscapeTemplate)

UObject* UDatasmithLandscapeTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	ALandscape* Landscape = UDatasmithActorTemplate::GetActor< ALandscape >( Destination );

	if( !Landscape )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithLandscapeTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithLandscapeTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET(LandscapeMaterial, Landscape, PreviousTemplate);
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET(StaticLightingLOD, Landscape, PreviousTemplate);
#endif // #if WITH_EDITORONLY_DATA

	return Landscape->GetRootComponent();
}

void UDatasmithLandscapeTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const ALandscape* Landscape = UDatasmithActorTemplate::GetActor< ALandscape >( Source );

	if( !Landscape )
	{
		return;
	}

	LandscapeMaterial = Landscape->LandscapeMaterial;
	StaticLightingLOD = Landscape->StaticLightingLOD;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithLandscapeTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithLandscapeTemplate* TypedOther = Cast< UDatasmithLandscapeTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = ( LandscapeMaterial == TypedOther->LandscapeMaterial );
	bEquals = bEquals && ( StaticLightingLOD == TypedOther->StaticLightingLOD );

	return bEquals;
}
