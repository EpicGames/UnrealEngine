// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithLightComponentTemplate.h"

#include "Components/LightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithLightComponentTemplate)

UDatasmithLightComponentTemplate::UDatasmithLightComponentTemplate()
{
}

UObject* UDatasmithLightComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	ULightComponent* LightComponent = Cast< ULightComponent >( Destination );

	if ( !LightComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithLightComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithLightComponentTemplate >( Destination ) : nullptr;

	if ( !PreviousTemplate || LightComponent->IsVisible() == PreviousTemplate->bVisible )
	{
		LightComponent->SetVisibility( bVisible );
	}

	if ( !PreviousTemplate || LightComponent->Intensity == PreviousTemplate->Intensity )
	{
		LightComponent->SetIntensity( Intensity );
	}

	if ( !PreviousTemplate || LightComponent->CastShadows == PreviousTemplate->CastShadows )
	{
		LightComponent->SetCastShadows( CastShadows );
	}

	if ( !PreviousTemplate || LightComponent->LightColor == PreviousTemplate->LightColor.ToFColor( true ) )
	{
		LightComponent->SetLightColor( LightColor );
	}

	if ( !PreviousTemplate || LightComponent->LightFunctionMaterial == PreviousTemplate->LightFunctionMaterial )
	{
		LightComponent->SetLightFunctionMaterial( LightFunctionMaterial );
	}

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUseTemperature, LightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Temperature, LightComponent, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( IESTexture, LightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUseIESBrightness, LightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( IESBrightnessScale, LightComponent, PreviousTemplate );
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithLightComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const ULightComponent* LightComponent = Cast< ULightComponent >( Source );

	if ( !LightComponent )
	{
		return;
	}

	bVisible = LightComponent->IsVisible();
	Intensity = LightComponent->Intensity;
	CastShadows = LightComponent->CastShadows;
	LightColor = LightComponent->LightColor;
	LightFunctionMaterial = LightComponent->LightFunctionMaterial;

	bUseTemperature = LightComponent->bUseTemperature;
	Temperature = LightComponent->Temperature;

	IESTexture = LightComponent->IESTexture;
	bUseIESBrightness = LightComponent->bUseIESBrightness;
	IESBrightnessScale = LightComponent->IESBrightnessScale;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithLightComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithLightComponentTemplate* TypedOther = Cast< UDatasmithLightComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = bVisible == TypedOther->bVisible;
	bEquals = bEquals && FMath::IsNearlyEqual( Intensity, TypedOther->Intensity );
	bEquals = bEquals && ( CastShadows == TypedOther->CastShadows );
	bEquals = bEquals && LightColor.Equals( TypedOther->LightColor );
	bEquals = bEquals && ( LightFunctionMaterial == TypedOther->LightFunctionMaterial );

	bEquals = bEquals && ( bUseTemperature == TypedOther->bUseTemperature );
	bEquals = bEquals && FMath::IsNearlyEqual( Temperature, TypedOther->Temperature );

	bEquals = bEquals && ( bUseIESBrightness == TypedOther->bUseIESBrightness );
	bEquals = bEquals && FMath::IsNearlyEqual( IESBrightnessScale, TypedOther->IESBrightnessScale );

	return bEquals;
}
