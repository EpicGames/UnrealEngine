// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithMaterialInstanceTemplate.h"

#include "Engine/Texture.h"
#include "Materials/MaterialInstanceConstant.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithMaterialInstanceTemplate)

namespace DatasmithMaterialInstanceTemplateImpl
{
#if WITH_EDITORONLY_DATA
	void Apply( UMaterialInstanceConstant* MaterialInstance, FName ParameterName, float Value, TOptional< float > PreviousValue )
	{
		float InstanceValue = 0.f;
		MaterialInstance->GetScalarParameterValue( ParameterName, InstanceValue );

		if ( PreviousValue && !FMath::IsNearlyEqual( InstanceValue, PreviousValue.GetValue() ) )
		{
			return;
		}

		if ( !FMath::IsNearlyEqual( Value, InstanceValue ) )
		{
			MaterialInstance->SetScalarParameterValueEditorOnly( ParameterName, Value );
		}
	}

	void Apply( UMaterialInstanceConstant* MaterialInstance, FName ParameterName, FLinearColor Value, TOptional< FLinearColor > PreviousValue )
	{
		FLinearColor InstanceValue = FLinearColor::White;
		MaterialInstance->GetVectorParameterValue( ParameterName, InstanceValue );

		if ( PreviousValue && !InstanceValue.Equals( PreviousValue.GetValue() ) )
		{
			return;
		}

		if ( !Value.Equals( InstanceValue ) )
		{
			MaterialInstance->SetVectorParameterValueEditorOnly( ParameterName, Value );
		}
	}

	void Apply( UMaterialInstanceConstant* MaterialInstance, FName ParameterName, TSoftObjectPtr< UTexture > Value, TOptional< TSoftObjectPtr< UTexture > > PreviousValue )
	{
		UTexture* InstanceValue = nullptr;
		MaterialInstance->GetTextureParameterValue( ParameterName, InstanceValue );

		if ( PreviousValue && InstanceValue != PreviousValue.GetValue() )
		{
			return;
		}

		if ( Value != InstanceValue )
		{
			MaterialInstance->SetTextureParameterValueEditorOnly( ParameterName, Value.Get() );
		}
	}
#endif // #if WITH_EDITORONLY_DATA

	template< typename MapType >
	bool MapEquals( MapType A, MapType B )
	{
		bool bEquals = ( A.Num() == B.Num() );

		for ( typename MapType::TConstIterator It = A.CreateConstIterator(); It && bEquals; ++It )
		{
			const auto* BValue = B.Find( It->Key );

			if ( BValue )
			{
				bEquals = ( It->Value == *BValue );
			}
			else
			{
				bEquals = false;
			}
		}

		return bEquals;
	}
}

void FDatasmithStaticParameterSetTemplate::Apply( UMaterialInstanceConstant* Destination, FDatasmithStaticParameterSetTemplate* PreviousTemplate )
{
#if WITH_EDITORONLY_DATA
	bool bNeedsUpdatePermutations = false;

	FStaticParameterSet DestinationStaticParameters;
	Destination->GetStaticParameterValues( DestinationStaticParameters );

	for ( TMap< FName, bool >::TConstIterator It = StaticSwitchParameters.CreateConstIterator(); It; ++It )
	{
		TOptional< bool > PreviousValue;

		if ( PreviousTemplate )
		{
			bool* PreviousValuePtr = PreviousTemplate->StaticSwitchParameters.Find( It->Key );

			if ( PreviousValuePtr )
			{
				PreviousValue = *PreviousValuePtr;
			}
		}

		for ( FStaticSwitchParameter& DestinationSwitchParameter : DestinationStaticParameters.StaticSwitchParameters )
		{
			if ( DestinationSwitchParameter.ParameterInfo.Name == It->Key )
			{
				if ( ( !PreviousValue || PreviousValue.GetValue() == DestinationSwitchParameter.Value ) && DestinationSwitchParameter.Value != It->Value )
				{
					DestinationSwitchParameter.Value = It->Value;
					DestinationSwitchParameter.bOverride = true;
					bNeedsUpdatePermutations = true;
				}

				break;
			}
		}
	}

	if ( bNeedsUpdatePermutations )
	{
		Destination->UpdateStaticPermutation( DestinationStaticParameters );
	}
#endif // #if WITH_EDITORONLY_DATA
}

void FDatasmithStaticParameterSetTemplate::Load( const UMaterialInstanceConstant& Source, bool bOverridesOnly )
{
#if WITH_EDITORONLY_DATA
	FStaticParameterSet SourceStaticParameters;
	const_cast< UMaterialInstanceConstant& >( Source ).GetStaticParameterValues( SourceStaticParameters );

	StaticSwitchParameters.Empty( SourceStaticParameters.StaticSwitchParameters.Num() );

	for ( const FStaticSwitchParameter& SourceSwitch : SourceStaticParameters.StaticSwitchParameters )
	{
		if ( !bOverridesOnly || SourceSwitch.bOverride )
		{
			StaticSwitchParameters.Add( SourceSwitch.ParameterInfo.Name, SourceSwitch.Value );
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}

void FDatasmithStaticParameterSetTemplate::LoadRebase(const UMaterialInstanceConstant& Source, const FDatasmithStaticParameterSetTemplate& ComparedTemplate, const FDatasmithStaticParameterSetTemplate* MergedTemplate)
{
#if WITH_EDITORONLY_DATA
	StaticSwitchParameters.Empty(ComparedTemplate.StaticSwitchParameters.Num());

	for (TMap< FName, bool >::TConstIterator It = ComparedTemplate.StaticSwitchParameters.CreateConstIterator(); It; ++It)
	{
		bool DefaultValue;
		FGuid ExpressionGuid;
		if (Source.GetStaticSwitchParameterDefaultValue(It->Key, DefaultValue, ExpressionGuid))
		{
			bool OldValue = It->Value;
			bool NewValue = DefaultValue;

			if (MergedTemplate)
			{
				if (const bool* BaseTemplateValue = MergedTemplate->StaticSwitchParameters.Find(It->Key))
				{
					NewValue = *BaseTemplateValue;
				}
			}

			// Store new value in case it's different from old or default is different from old
			if (NewValue != OldValue || DefaultValue != OldValue)
			{
				StaticSwitchParameters.Add(It->Key) = NewValue;
			}
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}


bool FDatasmithStaticParameterSetTemplate::Equals( const FDatasmithStaticParameterSetTemplate& Other ) const
{
	return DatasmithMaterialInstanceTemplateImpl::MapEquals( StaticSwitchParameters, Other.StaticSwitchParameters );
}

UObject* UDatasmithMaterialInstanceTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	UMaterialInstanceConstant* MaterialInstance = Cast< UMaterialInstanceConstant >( Destination );

	if ( !MaterialInstance )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithMaterialInstanceTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithMaterialInstanceTemplate >( MaterialInstance ) : nullptr;

	if (ParentMaterial.IsValid() && MaterialInstance->Parent != ParentMaterial.Get())
	{
		MaterialInstance->SetParentEditorOnly(ParentMaterial.Get(), false);
	}

	if ( !PreviousTemplate )
	{
		MaterialInstance->ClearParameterValuesEditorOnly(); // If we're not applying a delta (changes vs previous template), we start with a clean slate
	}

	for ( TMap< FName, float >::TConstIterator It = ScalarParameterValues.CreateConstIterator(); It; ++It )
	{
		TOptional< float > PreviousValue;

		if ( PreviousTemplate )
		{
			float* PreviousValuePtr = PreviousTemplate->ScalarParameterValues.Find( It->Key );

			if ( PreviousValuePtr )
			{
				PreviousValue = *PreviousValuePtr;
			}
		}

		DatasmithMaterialInstanceTemplateImpl::Apply( MaterialInstance, It->Key, It->Value, PreviousValue );
	}

	for ( TMap< FName, FLinearColor >::TConstIterator It = VectorParameterValues.CreateConstIterator(); It; ++It )
	{
		TOptional< FLinearColor > PreviousValue;

		if ( PreviousTemplate )
		{
			FLinearColor* PreviousValuePtr = PreviousTemplate->VectorParameterValues.Find( It->Key );

			if ( PreviousValuePtr )
			{
				PreviousValue = *PreviousValuePtr;
			}
		}

		DatasmithMaterialInstanceTemplateImpl::Apply( MaterialInstance, It->Key, It->Value, PreviousValue );
	}

	for ( TMap< FName, TSoftObjectPtr< UTexture > >::TConstIterator It = TextureParameterValues.CreateConstIterator(); It; ++It )
	{
		TOptional< TSoftObjectPtr< UTexture > > PreviousValue;

		if ( PreviousTemplate )
		{
			TSoftObjectPtr< UTexture >* PreviousValuePtr = PreviousTemplate->TextureParameterValues.Find( It->Key );

			if ( PreviousValuePtr )
			{
				PreviousValue = *PreviousValuePtr;
			}
		}

		DatasmithMaterialInstanceTemplateImpl::Apply( MaterialInstance, It->Key, It->Value, PreviousValue );
	}

	StaticParameters.Apply( MaterialInstance, PreviousTemplate ? &PreviousTemplate->StaticParameters : nullptr );
#endif // #if WITH_EDITORONLY_DATA

	return MaterialInstance;
}

void UDatasmithMaterialInstanceTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UMaterialInstanceConstant* MaterialInstance = Cast< UMaterialInstanceConstant >( Source );

	if ( !MaterialInstance )
	{
		return;
	}

	ParentMaterial = MaterialInstance->Parent;

	// Scalar
	ScalarParameterValues.Empty( MaterialInstance->ScalarParameterValues.Num() );

	for ( const FScalarParameterValue& ScalarParameterValue : MaterialInstance->ScalarParameterValues )
	{
		float Value;
		if ( MaterialInstance->GetScalarParameterValue( ScalarParameterValue.ParameterInfo.Name, Value, true ) )
		{
			ScalarParameterValues.Add( ScalarParameterValue.ParameterInfo.Name, Value );
		}
	}

	// Vector
	VectorParameterValues.Empty( MaterialInstance->VectorParameterValues.Num() );

	for ( const FVectorParameterValue& VectorParameterValue : MaterialInstance->VectorParameterValues )
	{
		FLinearColor Value;
		if ( MaterialInstance->GetVectorParameterValue( VectorParameterValue.ParameterInfo.Name, Value, true ) )
		{
			VectorParameterValues.Add( VectorParameterValue.ParameterInfo.Name, Value );
		}
	}

	// Texture
	TextureParameterValues.Empty( MaterialInstance->TextureParameterValues.Num() );

	for ( const FTextureParameterValue& TextureParameterValue : MaterialInstance->TextureParameterValues )
	{
		UTexture* Value;
		if ( MaterialInstance->GetTextureParameterValue( TextureParameterValue.ParameterInfo.Name, Value, true ) )
		{
			TextureParameterValues.Add( TextureParameterValue.ParameterInfo.Name, Value );
		}
	}

	StaticParameters.Load( *MaterialInstance );
#endif // #if WITH_EDITORONLY_DATA
}

void UDatasmithMaterialInstanceTemplate::LoadRebase(const UObject* Source, const UDatasmithObjectTemplate* BaseTemplate, bool bMergeTemplate)
{
#if WITH_EDITORONLY_DATA
	const UMaterialInstanceConstant* MaterialInstance = Cast< UMaterialInstanceConstant >(Source);
	const UDatasmithMaterialInstanceTemplate* TypedBaseTemplate = Cast< UDatasmithMaterialInstanceTemplate >(BaseTemplate);

	if (!MaterialInstance || !TypedBaseTemplate)
	{
		return;
	}

	// Load the full template of the Source object, we need it because some parameters default values might have changed with the different ParentMaterial.
	TStrongObjectPtr< UDatasmithMaterialInstanceTemplate > DestinationTemplate{ NewObject< UDatasmithMaterialInstanceTemplate >(GetTransientPackage()) };
	DestinationTemplate->LoadAll(Source);

	ParentMaterial = TypedBaseTemplate->ParentMaterial;

	ScalarParameterValues.Empty(DestinationTemplate->ScalarParameterValues.Num());
	for (TMap< FName, float >::TConstIterator It = DestinationTemplate->ScalarParameterValues.CreateConstIterator(); It; ++It)
	{
		float DefaultValue;
		if (ParentMaterial->GetScalarParameterDefaultValue(It->Key, DefaultValue))
		{
			float OldValue = It->Value;
			float NewValue = DefaultValue;

			if (bMergeTemplate)
			{
				if (const float* BaseTemplateValue = TypedBaseTemplate->ScalarParameterValues.Find(It->Key))
				{
					NewValue = *BaseTemplateValue;
				}
			}

			// Store new value in case it's different from old or default is different from old
			if (NewValue != OldValue || DefaultValue != OldValue)
			{
				ScalarParameterValues.Add(It->Key) = NewValue;
			}
		}
	}

	VectorParameterValues.Empty(DestinationTemplate->VectorParameterValues.Num());
	for (TMap< FName, FLinearColor >::TConstIterator It = DestinationTemplate->VectorParameterValues.CreateConstIterator(); It; ++It)
	{
		FLinearColor DefaultValue;
		if (ParentMaterial->GetVectorParameterDefaultValue(It->Key, DefaultValue))
		{
			FLinearColor OldValue = It->Value;
			FLinearColor NewValue = DefaultValue;
			
			if (bMergeTemplate)
			{
				if (const FLinearColor* BaseTemplateValue = TypedBaseTemplate->VectorParameterValues.Find(It->Key))
				{
					NewValue = *BaseTemplateValue;
				}
			}

			// Store new value in case it's different from old or default is different from old
			if (NewValue != OldValue || DefaultValue != OldValue)
			{
				VectorParameterValues.Add(It->Key) = NewValue;
			}
		}
	}

	TextureParameterValues.Empty(DestinationTemplate->TextureParameterValues.Num());
	for (TMap< FName, TSoftObjectPtr< UTexture > >::TConstIterator It = DestinationTemplate->TextureParameterValues.CreateConstIterator(); It; ++It)
	{
		UTexture* DefaultValue;
		if (ParentMaterial->GetTextureParameterDefaultValue(It->Key, DefaultValue))
		{
			UTexture* OldValue = It->Value.Get();
			UTexture* NewValue = DefaultValue;

			if (bMergeTemplate)
			{
				if (const TSoftObjectPtr<UTexture>* BaseTemplateValue = TypedBaseTemplate->TextureParameterValues.Find(It->Key))
				{
					NewValue = BaseTemplateValue->Get();
				}
			}
			
			// Store new value in case it's different from old or default is different from old
			if (NewValue != OldValue || DefaultValue != OldValue)
			{
				TextureParameterValues.Add(It->Key) = NewValue;
			}
		}
	}

	StaticParameters.LoadRebase(*MaterialInstance, DestinationTemplate->StaticParameters, bMergeTemplate ? &TypedBaseTemplate->StaticParameters : nullptr);
#endif // #if WITH_EDITORONLY_DATA
}

void UDatasmithMaterialInstanceTemplate::LoadAll( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UMaterialInstanceConstant* MaterialInstance = Cast< UMaterialInstanceConstant >(Source);

	if ( !MaterialInstance )
	{
		return;
	}
	ParentMaterial = MaterialInstance->Parent;

	TArray<FMaterialParameterInfo> MaterialParameterInfos;
	TArray<FGuid> MaterialParameterGuids;

	// Scalar
	ScalarParameterValues.Empty(MaterialInstance->ScalarParameterValues.Num());
	MaterialInstance->GetAllScalarParameterInfo(MaterialParameterInfos, MaterialParameterGuids);

	for ( const FMaterialParameterInfo& ScalarParameterInfo : MaterialParameterInfos )
	{
		float Value;
		if ( MaterialInstance->GetScalarParameterValue( ScalarParameterInfo.Name, Value ) )
		{
			ScalarParameterValues.Add( ScalarParameterInfo.Name, Value );
		}
	}

	// Vector
	VectorParameterValues.Empty( MaterialInstance->VectorParameterValues.Num() );
	MaterialInstance->GetAllVectorParameterInfo( MaterialParameterInfos, MaterialParameterGuids );

	for ( const FMaterialParameterInfo& VectorParameterInfo : MaterialParameterInfos )
	{
		FLinearColor Value;
		if ( MaterialInstance->GetVectorParameterValue( VectorParameterInfo.Name, Value ) )
		{
			VectorParameterValues.Add( VectorParameterInfo.Name, Value );
		}
	}

	// Texture
	TextureParameterValues.Empty( MaterialInstance->TextureParameterValues.Num() );
	MaterialInstance->GetAllTextureParameterInfo( MaterialParameterInfos, MaterialParameterGuids );

	for ( const FMaterialParameterInfo& TextureParameterInfo : MaterialParameterInfos )
	{
		UTexture* Value;
		if ( MaterialInstance->GetTextureParameterValue( TextureParameterInfo.Name, Value ) )
		{
			TextureParameterValues.Add( TextureParameterInfo.Name, Value );
		}
	}

	StaticParameters.Load( *MaterialInstance, false );
#endif // #if WITH_EDITORONLY_DATA
}


bool UDatasmithMaterialInstanceTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithMaterialInstanceTemplate* TypedOther = Cast< UDatasmithMaterialInstanceTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = TypedOther->ParentMaterial.Get() == ParentMaterial.Get();
	bEquals = bEquals && DatasmithMaterialInstanceTemplateImpl::MapEquals( ScalarParameterValues, TypedOther->ScalarParameterValues );
	bEquals = bEquals && DatasmithMaterialInstanceTemplateImpl::MapEquals( VectorParameterValues, TypedOther->VectorParameterValues );
	bEquals = bEquals && DatasmithMaterialInstanceTemplateImpl::MapEquals( TextureParameterValues, TypedOther->TextureParameterValues );

	bEquals = bEquals && StaticParameters.Equals( TypedOther->StaticParameters );

	return bEquals;
}

bool UDatasmithMaterialInstanceTemplate::HasSameBase(const UDatasmithObjectTemplate* Other) const
{
	const UDatasmithMaterialInstanceTemplate* TypedOther = Cast< UDatasmithMaterialInstanceTemplate >(Other);

	if (!TypedOther)
	{
		return false;
	}

	return ParentMaterial == TypedOther->ParentMaterial;
}
