// Copyright Epic Games, Inc. All Rights Reserved.
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithAssetUserData.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithObjectTemplate)


namespace DatasmithObjectTemplateUtilsImpl
{
	IInterface_AssetUserData* GetUserDataInterface(UObject* Outer)
	{
		if (Outer && Outer->GetClass()->IsChildOf(AActor::StaticClass()))
		{
			// The root Component holds AssetUserData on behalf of the actor
			Outer = Cast<AActor>(Outer)->GetRootComponent();
		}

		if (!Outer || !Outer->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
		{
			return nullptr;
		}

		return Cast< IInterface_AssetUserData >(Outer);
	}
}

bool FDatasmithObjectTemplateUtils::HasObjectTemplates(UObject* Outer)
{
#if WITH_EDITORONLY_DATA
	IInterface_AssetUserData* AssetUserDataInterface = DatasmithObjectTemplateUtilsImpl::GetUserDataInterface(Outer);
	if (!AssetUserDataInterface)
	{
		return false;
	}

	UDatasmithAssetUserData* UserData = AssetUserDataInterface->GetAssetUserData< UDatasmithAssetUserData >();

	return UserData != nullptr && UserData->ObjectTemplates.Num() > 0;
#else
	return false;
#endif // #if WITH_EDITORONLY_DATA
}

UDatasmithAssetUserData* FDatasmithObjectTemplateUtils::FindOrCreateDatasmithUserData(UObject* Outer)
{
	IInterface_AssetUserData* AssetUserDataInterface = DatasmithObjectTemplateUtilsImpl::GetUserDataInterface(Outer);
	if (!AssetUserDataInterface)
	{
		return nullptr;
	}

	UDatasmithAssetUserData* UserData = AssetUserDataInterface->GetAssetUserData< UDatasmithAssetUserData >();

	if (!UserData)
	{
		EObjectFlags Flags = RF_Public /*| RF_Transactional*/; // RF_Transactional Disabled as is can cause a crash in the transaction system for blueprints

		if ( Outer->GetClass()->IsChildOf<AActor>() )
		{
			// The outer should never be an actor. (UE-70039)
			Outer = static_cast<AActor*>(Outer)->GetRootComponent();
		}

		UserData = NewObject< UDatasmithAssetUserData >(Outer, NAME_None, Flags);
		AssetUserDataInterface->AddAssetUserData(UserData);
	}

	return UserData;
}

TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >* FDatasmithObjectTemplateUtils::FindOrCreateObjectTemplates(UObject* Outer)
{
#if WITH_EDITORONLY_DATA
	if (UDatasmithAssetUserData* UserAssetData = FindOrCreateDatasmithUserData(Outer))
	{
		return &UserAssetData->ObjectTemplates;
	}
#endif // #if WITH_EDITORONLY_DATA
return nullptr;
}

void UDatasmithObjectTemplate::Apply(UObject* Destination, bool bForce)
{
#if WITH_EDITORONLY_DATA
	if(UObject* Object = UpdateObject(Destination, bForce))
	{
		FDatasmithObjectTemplateUtils::SetObjectTemplate(Object, this);
	}
#endif // #if WITH_EDITORONLY_DATA
}

UDatasmithObjectTemplate* FDatasmithObjectTemplateUtils::GetObjectTemplate(UObject* Outer, TSubclassOf< UDatasmithObjectTemplate > Subclass)
{
#if WITH_EDITORONLY_DATA
	TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >* ObjectTemplatesMap = FindOrCreateObjectTemplates(Outer);

	if (!ObjectTemplatesMap)
	{
		return nullptr;
	}

	TObjectPtr<UDatasmithObjectTemplate>* ObjectTemplatePtr = ObjectTemplatesMap->Find(Subclass);

	return ObjectTemplatePtr ? *ObjectTemplatePtr : nullptr;
#else
	return nullptr;
#endif // #if WITH_EDITORONLY_DATA
}

void FDatasmithObjectTemplateUtils::SetObjectTemplate(UObject* Outer, UDatasmithObjectTemplate* ObjectTemplate)
{
#if WITH_EDITORONLY_DATA
	if (UDatasmithAssetUserData* UserData = FindOrCreateDatasmithUserData(Outer))
	{
		TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >& ObjectTemplatesMap = UserData->ObjectTemplates;
		ObjectTemplatesMap.FindOrAdd(ObjectTemplate->GetClass()) = ObjectTemplate;
		ObjectTemplate->SetFlags(RF_Public);

		if (ObjectTemplate->GetOuter() != UserData)
		{
			// The outer chain is important for most of the engine functionality.
			// If it's not set properly the deep copy of object won't work properly
			ObjectTemplate->Rename(nullptr, UserData, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}

TSet<FName> FDatasmithObjectTemplateUtils::ThreeWaySetMerge(const TSet<FName>& OldSet, const TSet<FName>& CurrentSet, const TSet<FName>& NewSet)
{
	TSet<FName> UserRemoved = OldSet.Difference(CurrentSet);
	TSet<FName> UserAdded = CurrentSet.Difference(OldSet);
	return NewSet.Union(UserAdded).Difference(UserRemoved);
}

bool FDatasmithObjectTemplateUtils::SetsEquals(const TSet<FName>& Left, const TSet<FName>& Right)
{
	return Left.Num() == Right.Num() && Left.Includes(Right);
}

UDatasmithObjectTemplate* UDatasmithObjectTemplate::GetDifference(UObject* Destination, UDatasmithObjectTemplate* SourceTemplate)
{
	// Cache the template of the Destination object
	TStrongObjectPtr< UDatasmithObjectTemplate > DestinationTemplate{ NewObject< UDatasmithObjectTemplate >(GetTransientPackage(), SourceTemplate->GetClass()) };
	DestinationTemplate->Load(Destination);

	if ( !SourceTemplate->HasSameBase(DestinationTemplate.Get()) )
	{
		//The 2 templates don't have the same base object, we need to load the rebased template instead.
		UDatasmithObjectTemplate* DiffTemplate = NewObject< UDatasmithObjectTemplate >(GetTransientPackage(), SourceTemplate->GetClass());
		DiffTemplate->LoadRebase(Destination, SourceTemplate, true);

		return DiffTemplate;
	}

	// Update the Destination object with the new template
	SourceTemplate->UpdateObject(Destination);

	// Create a new template based on the updated Destination object
	UDatasmithObjectTemplate* DiffTemplate = NewObject< UDatasmithObjectTemplate >(GetTransientPackage(), SourceTemplate->GetClass());
	DiffTemplate->Load(Destination);

	// Restore Destination object to previous state
	DestinationTemplate->UpdateObject(Destination, true);

	return DiffTemplate;
}
