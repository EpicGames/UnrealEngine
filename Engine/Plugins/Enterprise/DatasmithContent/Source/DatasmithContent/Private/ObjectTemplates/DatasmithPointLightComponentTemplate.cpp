// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithPointLightComponentTemplate.h"

#include "Components/PointLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithPointLightComponentTemplate)

UDatasmithPointLightComponentTemplate::UDatasmithPointLightComponentTemplate()
{
	UDatasmithPointLightComponentTemplate::Load( UPointLightComponent::StaticClass()->GetDefaultObject() );
}

UObject* UDatasmithPointLightComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	UPointLightComponent* PointLightComponent = Cast< UPointLightComponent >( Destination );

	if ( !PointLightComponent )
	{
		return nullptr;
	}
#if WITH_EDITORONLY_DATA

	UDatasmithPointLightComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithPointLightComponentTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( IntensityUnits, PointLightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SourceRadius, PointLightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SourceLength, PointLightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( AttenuationRadius, PointLightComponent, PreviousTemplate );
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithPointLightComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UPointLightComponent* PointLightComponent = Cast< UPointLightComponent >( Source );

	if ( !PointLightComponent )
	{
		return;
	}

	IntensityUnits = PointLightComponent->IntensityUnits;
	SourceRadius = PointLightComponent->SourceRadius;
	SourceLength = PointLightComponent->SourceLength;
	AttenuationRadius = PointLightComponent->AttenuationRadius;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithPointLightComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithPointLightComponentTemplate* TypedOther = Cast< UDatasmithPointLightComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = IntensityUnits == TypedOther->IntensityUnits;
	bEquals = bEquals && FMath::IsNearlyEqual( SourceRadius, TypedOther->SourceRadius );
	bEquals = bEquals && FMath::IsNearlyEqual( SourceLength, TypedOther->SourceLength );
	bEquals = bEquals && FMath::IsNearlyEqual( AttenuationRadius, TypedOther->AttenuationRadius );

	return bEquals;
}
