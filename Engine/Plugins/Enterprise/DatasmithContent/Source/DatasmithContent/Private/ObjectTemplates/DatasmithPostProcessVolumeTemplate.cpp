// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithPostProcessVolumeTemplate.h"

#include "ObjectTemplates/DatasmithActorTemplate.h"

#include "Engine/PostProcessVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithPostProcessVolumeTemplate)

UObject* UDatasmithPostProcessVolumeTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	APostProcessVolume* PostProcessVolume = UDatasmithActorTemplate::GetActor< APostProcessVolume >( Destination );

	if ( !PostProcessVolume )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithPostProcessVolumeTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithPostProcessVolumeTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bEnabled, PostProcessVolume, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUnbound, PostProcessVolume, PreviousTemplate );

	Settings.Apply( &PostProcessVolume->Settings, PreviousTemplate ? &PreviousTemplate->Settings : nullptr );
#endif // #if WITH_EDITORONLY_DATA

	return PostProcessVolume->GetRootComponent();
}

void UDatasmithPostProcessVolumeTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const APostProcessVolume* PostProcessVolume = UDatasmithActorTemplate::GetActor< const APostProcessVolume >( Source );
	if ( !PostProcessVolume )
	{
		return;
	}

	bEnabled = PostProcessVolume->bEnabled;
	bUnbound = PostProcessVolume->bUnbound;

	Settings.Load( PostProcessVolume->Settings );
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithPostProcessVolumeTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithPostProcessVolumeTemplate* TypedOther = Cast< UDatasmithPostProcessVolumeTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = ( bEnabled == TypedOther->bEnabled );
	bEquals = bEquals && ( bUnbound == TypedOther->bUnbound );
	bEquals = bEquals && Settings.Equals( TypedOther->Settings );

	return bEquals;
}
