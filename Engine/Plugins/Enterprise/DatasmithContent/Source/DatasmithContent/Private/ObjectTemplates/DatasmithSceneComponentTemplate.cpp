// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithSceneComponentTemplate.h"

#include "DatasmithAssetUserData.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithSceneComponentTemplate)

namespace
{
	bool AreTransformsEqual( const FTransform& A, const FTransform& B )
	{
		return A.TranslationEquals( B, THRESH_POINTS_ARE_NEAR ) && A.RotationEquals( B, KINDA_SMALL_NUMBER) && A.Scale3DEquals( B, KINDA_SMALL_NUMBER );
	}
}

UObject* UDatasmithSceneComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	USceneComponent* SceneComponent = Cast< USceneComponent >( Destination );

	if ( !SceneComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithSceneComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithSceneComponentTemplate >( Destination ) : nullptr;

	if ( !PreviousTemplate || PreviousTemplate->Mobility == SceneComponent->Mobility )
	{
		// We can't attach a static component to a dynamic parent
		EComponentMobility::Type TargetMobility = Mobility;
		if (AttachParent)
		{
			TargetMobility = (EComponentMobility::Type) FMath::Max((uint8)Mobility, (uint8)AttachParent->Mobility);
		}

		SceneComponent->SetMobility( TargetMobility );
	}

	if ( !PreviousTemplate || PreviousTemplate->bVisible == SceneComponent->GetVisibleFlag() )
	{
		SceneComponent->SetVisibility( bVisible );
	}

	if ( UPrimitiveComponent* PrimComponent = Cast< UPrimitiveComponent >( SceneComponent ) )
	{
		if ( !PreviousTemplate || PreviousTemplate->bCastShadow == PrimComponent->CastShadow )
		{
			PrimComponent->SetCastShadow( bCastShadow );
		}
	}

	const ULevel* SceneComponentLevel = SceneComponent->GetComponentLevel();
	bool bCanAttach = ( AttachParent && AttachParent->GetComponentLevel() == SceneComponentLevel );
	bCanAttach |= ( !AttachParent && ( SceneComponentLevel == nullptr || SceneComponentLevel->OwningWorld == Destination->GetWorld() ) );

	if ( !PreviousTemplate || PreviousTemplate->AttachParent == SceneComponent->GetAttachParent() )
	{
		FAttachmentTransformRules AttachmentTransformRules = FAttachmentTransformRules::KeepRelativeTransform;

		// We assume that all Datasmith components were created with a parent.
		// If we already have a component template but no parent, it means that we got detached since the last import,
		// in which case we want to keep the world position when reattaching.
		const bool bLostItsParent = PreviousTemplate && PreviousTemplate->AttachParent == nullptr;

		if ( bLostItsParent )
		{
			AttachmentTransformRules = FAttachmentTransformRules::KeepWorldTransform;
		}

		if ( bCanAttach )
		{
			if ( AttachParent )
			{
				SceneComponent->AttachToComponent( AttachParent.Get(), AttachmentTransformRules );
			}
			// If AtachParent is null, the owning actor is at the root of the world
			// Just detach it from its current SceneComponent
			else if( AActor* ParentActor = SceneComponent->GetTypedOuter< AActor >() )
			{
				SceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
			}
		}
	}

	if ( !PreviousTemplate || AreTransformsEqual( PreviousTemplate->RelativeTransform, SceneComponent->GetRelativeTransform() ) )
	{
		if ( bCanAttach )
		{
			SceneComponent->SetRelativeTransform( RelativeTransform );
		}
		else
		{
			// We were unable to attach to our parent so we need to compute the desired world transform
			FTransform WorldTransform = RelativeTransform;

			if ( AttachParent )
			{
				WorldTransform *= AttachParent->GetComponentTransform();
			}

			SceneComponent->SetRelativeTransform( WorldTransform );
		}
	}

	if ( !PreviousTemplate )
	{
		SceneComponent->ComponentTags = Tags.Array();
	}
	else
	{
		SceneComponent->ComponentTags = FDatasmithObjectTemplateUtils::ThreeWaySetMerge(PreviousTemplate->Tags, TSet<FName>(SceneComponent->ComponentTags), Tags).Array();
	}
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithSceneComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const USceneComponent* SceneComponent = Cast< USceneComponent >( Source );

	if ( !SceneComponent )
	{
		return;
	}

	RelativeTransform = SceneComponent->GetRelativeTransform();
	Mobility = SceneComponent->Mobility;
	bVisible = SceneComponent->GetVisibleFlag();
	AttachParent = SceneComponent->GetAttachParent();
	Tags = TSet<FName>(SceneComponent->ComponentTags);

	const UPrimitiveComponent* PrimComponent = Cast< const UPrimitiveComponent >( SceneComponent );
	bCastShadow = PrimComponent ? PrimComponent->CastShadow : true;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithSceneComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithSceneComponentTemplate* TypedOther = Cast< UDatasmithSceneComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = AreTransformsEqual( RelativeTransform, TypedOther->RelativeTransform );
	bEquals = bEquals && ( Mobility == TypedOther->Mobility );
	bEquals = bEquals && ( bVisible == TypedOther->bVisible );
	bEquals = bEquals && ( bCastShadow == TypedOther->bCastShadow );
	bEquals = bEquals && ( AttachParent == TypedOther->AttachParent );
	bEquals = bEquals && FDatasmithObjectTemplateUtils::SetsEquals(Tags, TypedOther->Tags);

	return bEquals;
}
