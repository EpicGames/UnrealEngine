// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithSkyLightComponentTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithSkyLightComponentTemplate)

UDatasmithSkyLightComponentTemplate::UDatasmithSkyLightComponentTemplate()
{
	UDatasmithSkyLightComponentTemplate::Load( USkyLightComponent::StaticClass()->GetDefaultObject() );
}

UObject* UDatasmithSkyLightComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	USkyLightComponent* SkyLightComponent = Cast< USkyLightComponent >( Destination );

	if ( !SkyLightComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithSkyLightComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithSkyLightComponentTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SourceType, SkyLightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( CubemapResolution, SkyLightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( Cubemap, SkyLightComponent, PreviousTemplate );
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithSkyLightComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const USkyLightComponent* SkyLightComponent = Cast< USkyLightComponent >( Source );

	if ( !SkyLightComponent )
	{
		return;
	}

	SourceType = SkyLightComponent->SourceType;
	CubemapResolution = SkyLightComponent->CubemapResolution;
	Cubemap = SkyLightComponent->Cubemap;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithSkyLightComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithSkyLightComponentTemplate* TypedOther = Cast< UDatasmithSkyLightComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = SourceType == TypedOther->SourceType;
	bEquals = bEquals && ( CubemapResolution == TypedOther->CubemapResolution );
	bEquals = bEquals && ( Cubemap == TypedOther->Cubemap );

	return bEquals;
}
