// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithSpotLightComponentTemplate.h"

#include "Components/SpotLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithSpotLightComponentTemplate)

UDatasmithSpotLightComponentTemplate::UDatasmithSpotLightComponentTemplate()
{
	UDatasmithSpotLightComponentTemplate::Load( USpotLightComponent::StaticClass()->GetDefaultObject() );
}

UObject* UDatasmithSpotLightComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	USpotLightComponent* SpotLightComponent = Cast< USpotLightComponent >( Destination );

	if ( !SpotLightComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithSpotLightComponentTemplate* PreviousTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithSpotLightComponentTemplate >( Destination ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( InnerConeAngle, SpotLightComponent, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( OuterConeAngle, SpotLightComponent, PreviousTemplate );
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithSpotLightComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const USpotLightComponent* SpotLightComponent = Cast< USpotLightComponent >( Source );

	if ( !SpotLightComponent )
	{
		return;
	}
	
	InnerConeAngle = SpotLightComponent->InnerConeAngle;
	OuterConeAngle = SpotLightComponent->OuterConeAngle;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithSpotLightComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithSpotLightComponentTemplate* TypedOther = Cast< UDatasmithSpotLightComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = FMath::IsNearlyEqual( InnerConeAngle, TypedOther->InnerConeAngle );
	bEquals = bEquals && FMath::IsNearlyEqual( OuterConeAngle, TypedOther->OuterConeAngle );

	return bEquals;
}
