// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithStaticMeshComponentTemplate.h"

#include "Components/StaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithStaticMeshComponentTemplate)

UObject* UDatasmithStaticMeshComponentTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	UStaticMeshComponent* StaticMeshComponent = Cast< UStaticMeshComponent >( Destination );

	if ( !StaticMeshComponent )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithStaticMeshComponentTemplate* PreviousStaticMeshTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithStaticMeshComponentTemplate >( Destination ) : nullptr;

	if ( !PreviousStaticMeshTemplate || PreviousStaticMeshTemplate->StaticMesh == StaticMeshComponent->GetStaticMesh() )
	{
		StaticMeshComponent->SetStaticMesh( StaticMesh );
	}

	if ( !PreviousStaticMeshTemplate )
	{
		StaticMeshComponent->OverrideMaterials.Empty( OverrideMaterials.Num() );
	}

	for ( int32 MaterialIndex = 0; MaterialIndex < OverrideMaterials.Num(); ++MaterialIndex )
	{
		UMaterialInterface* CurrentOverrideOnComponent = nullptr;
		if ( StaticMeshComponent->OverrideMaterials.IsValidIndex( MaterialIndex ) )
		{
			CurrentOverrideOnComponent = StaticMeshComponent->OverrideMaterials[ MaterialIndex ];
		}

		UMaterialInterface* PreviousTemplateOverride = nullptr;
		if ( PreviousStaticMeshTemplate && PreviousStaticMeshTemplate->OverrideMaterials.IsValidIndex( MaterialIndex ) )
		{
			PreviousTemplateOverride = PreviousStaticMeshTemplate->OverrideMaterials[ MaterialIndex ];
		}

		UMaterialInterface* NewTemplateOverride = OverrideMaterials[ MaterialIndex ];

		// There wasn't an override there before and there isn't a manually user-set override still (both nullptr): Set our new override.
		// Or there was an override there before but it's still there (both non-nullptr): The user hasn't changed anything, so we can set our new override
		if ( PreviousTemplateOverride == CurrentOverrideOnComponent )
		{
			StaticMeshComponent->SetMaterial( MaterialIndex, NewTemplateOverride );
		}
	}

	// Remove materials that aren't in the template anymore
	if ( PreviousStaticMeshTemplate )
	{
		for ( int32 MaterialIndexToRemove = PreviousStaticMeshTemplate->OverrideMaterials.Num() - 1; MaterialIndexToRemove >= OverrideMaterials.Num(); --MaterialIndexToRemove )
		{
			if ( StaticMeshComponent->OverrideMaterials.IsValidIndex( MaterialIndexToRemove ) &&
				StaticMeshComponent->OverrideMaterials[MaterialIndexToRemove] == PreviousStaticMeshTemplate->OverrideMaterials[MaterialIndexToRemove] )
			{
				StaticMeshComponent->OverrideMaterials.RemoveAt( MaterialIndexToRemove );
			}
		}
	}

	StaticMeshComponent->MarkRenderStateDirty();
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithStaticMeshComponentTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UStaticMeshComponent* SourceComponent = Cast< UStaticMeshComponent >( Source );

	if ( !SourceComponent )
	{
		return;
	}

	StaticMesh = SourceComponent->GetStaticMesh();
	OverrideMaterials = SourceComponent->OverrideMaterials;
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithStaticMeshComponentTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithStaticMeshComponentTemplate* TypedOther = Cast< UDatasmithStaticMeshComponentTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = StaticMesh == TypedOther->StaticMesh;
	bEquals = bEquals && ( OverrideMaterials == TypedOther->OverrideMaterials );

	return bEquals;
}
