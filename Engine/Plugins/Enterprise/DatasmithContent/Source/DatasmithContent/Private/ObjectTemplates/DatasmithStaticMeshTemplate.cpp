// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTemplates/DatasmithStaticMeshTemplate.h"

#include "DatasmithAssetUserData.h"

#include "CoreTypes.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshAttributes.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithStaticMeshTemplate)

FDatasmithMeshBuildSettingsTemplate::FDatasmithMeshBuildSettingsTemplate()
{
	Load ( FMeshBuildSettings() ); // Initialize from default object
}

void FDatasmithMeshBuildSettingsTemplate::Apply( FMeshBuildSettings* Destination, FDatasmithMeshBuildSettingsTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUseMikkTSpace, Destination, PreviousTemplate );

	// The settings for RecomputeNormals and RecomputeTangents when True must be honored irrespective of the previous template settings
	// because their values are determined by ShouldRecomputeNormals/ShouldRecomputeTangents which determine if they are needed by the renderer
	Destination->bRecomputeNormals = PreviousTemplate ? PreviousTemplate->bRecomputeNormals | bRecomputeNormals : bRecomputeNormals;

	Destination->bRecomputeTangents = PreviousTemplate ? PreviousTemplate->bRecomputeTangents | bRecomputeTangents : bRecomputeTangents;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bRemoveDegenerates, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUseHighPrecisionTangentBasis, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bUseFullPrecisionUVs, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( bGenerateLightmapUVs, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( MinLightmapResolution, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( SrcLightmapIndex, Destination, PreviousTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( DstLightmapIndex, Destination, PreviousTemplate );
}

void FDatasmithMeshBuildSettingsTemplate::Load( const FMeshBuildSettings& Source )
{
	bUseMikkTSpace = Source.bUseMikkTSpace;
	bRecomputeNormals = Source.bRecomputeNormals;
	bRecomputeTangents = Source.bRecomputeTangents;
	bRemoveDegenerates = Source.bRemoveDegenerates;
	bUseHighPrecisionTangentBasis = Source.bUseHighPrecisionTangentBasis;
	bUseFullPrecisionUVs = Source.bUseFullPrecisionUVs;
	bGenerateLightmapUVs = Source.bGenerateLightmapUVs;
	MinLightmapResolution = Source.MinLightmapResolution;
	SrcLightmapIndex = Source.SrcLightmapIndex;
	DstLightmapIndex = Source.DstLightmapIndex;
}

bool FDatasmithMeshBuildSettingsTemplate::Equals( const FDatasmithMeshBuildSettingsTemplate& Other ) const
{
	bool bEquals = bUseMikkTSpace == Other.bUseMikkTSpace;
	bEquals = bEquals && ( bRecomputeNormals == Other.bRecomputeNormals );
	bEquals = bEquals && ( bRecomputeTangents == Other.bRecomputeTangents );
	bEquals = bEquals && ( bRemoveDegenerates == Other.bRemoveDegenerates );
	bEquals = bEquals && ( bUseHighPrecisionTangentBasis == Other.bUseHighPrecisionTangentBasis );
	bEquals = bEquals && ( bUseFullPrecisionUVs == Other.bUseFullPrecisionUVs );
	bEquals = bEquals && ( bGenerateLightmapUVs == Other.bGenerateLightmapUVs );
	bEquals = bEquals && ( MinLightmapResolution == Other.MinLightmapResolution );
	bEquals = bEquals && ( SrcLightmapIndex == Other.SrcLightmapIndex );
	bEquals = bEquals && ( DstLightmapIndex == Other.DstLightmapIndex );

	return bEquals;
}

FDatasmithStaticMaterialTemplate::FDatasmithStaticMaterialTemplate()
{
	Load( FStaticMaterial() ); // Initialize from default object
}

void FDatasmithStaticMaterialTemplate::Apply( FStaticMaterial* Destination, FDatasmithStaticMaterialTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( MaterialSlotName, Destination, PreviousTemplate );
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( MaterialInterface, Destination, PreviousTemplate );

	Destination->ImportedMaterialSlotName = MaterialSlotName; // Not editable by the user, so always set it
}

void FDatasmithStaticMaterialTemplate::Load( const FStaticMaterial& Source )
{
	MaterialSlotName = Source.MaterialSlotName;
	MaterialInterface = Source.MaterialInterface;
}

bool FDatasmithStaticMaterialTemplate::Equals( const FDatasmithStaticMaterialTemplate& Other ) const
{
	bool bEquals = MaterialSlotName == Other.MaterialSlotName;
	bEquals = bEquals && ( MaterialInterface == Other.MaterialInterface );

	return bEquals;
}

FDatasmithMeshSectionInfoTemplate::FDatasmithMeshSectionInfoTemplate()
{
	Load( FMeshSectionInfo() ); // Initialize from default object
}

void FDatasmithMeshSectionInfoTemplate::Apply( FMeshSectionInfo* Destination, FDatasmithMeshSectionInfoTemplate* PreviousTemplate )
{
	DATASMITHOBJECTTEMPLATE_CONDITIONALSET( MaterialIndex, Destination, PreviousTemplate );
}

void FDatasmithMeshSectionInfoTemplate::Load( const FMeshSectionInfo& Source )
{
	MaterialIndex = Source.MaterialIndex;
}

bool FDatasmithMeshSectionInfoTemplate::Equals( const FDatasmithMeshSectionInfoTemplate& Other ) const
{
	return MaterialIndex == Other.MaterialIndex;
}

void FDatasmithMeshSectionInfoMapTemplate::Apply( FMeshSectionInfoMap* Destination, FDatasmithMeshSectionInfoMapTemplate* PreviousTemplate )
{
	for ( auto It = Map.CreateIterator(); It; ++It )
	{
		FMeshSectionInfo* SectionInfo = Destination->Map.Find( It->Key );
		if ( !SectionInfo )
		{
			//  If it wasn't found, must be added and the value forced on it
			SectionInfo = &Destination->Map.Add( It->Key );
			It->Value.Apply(SectionInfo, nullptr);
		}
		else
		{
			It->Value.Apply(SectionInfo, PreviousTemplate ? PreviousTemplate->Map.Find(It->Key) : nullptr);
		}
	}
}

void FDatasmithMeshSectionInfoMapTemplate::Load( const FMeshSectionInfoMap& Source )
{
	Map.Empty();

	for ( auto It = Source.Map.CreateConstIterator(); It; ++It )
	{
		FDatasmithMeshSectionInfoTemplate MeshSectionInfoTemplate;
		MeshSectionInfoTemplate.Load( It->Value );

		Map.Add( It->Key, MoveTemp( MeshSectionInfoTemplate ) );
	}
}

bool FDatasmithMeshSectionInfoMapTemplate::Equals( const FDatasmithMeshSectionInfoMapTemplate& Other ) const
{
	bool bEquals = ( Map.Num() == Other.Map.Num() );

	if ( bEquals )
	{
		for ( const auto& It : Map )
		{
			bEquals = bEquals && ( Other.Map.Contains( It.Key ) );

			if ( bEquals )
			{
				bEquals = bEquals && It.Value.Equals( *Other.Map.Find( It.Key ) );
			}

			if ( !bEquals )
			{
				break;
			}
		}
	}
	
	return bEquals;
}

UObject* UDatasmithStaticMeshTemplate::UpdateObject( UObject* Destination, bool bForce )
{
	UStaticMesh* StaticMesh = Cast< UStaticMesh >( Destination );

	if ( !StaticMesh )
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	UDatasmithStaticMeshTemplate* PreviousStaticMeshTemplate = !bForce ? FDatasmithObjectTemplateUtils::GetObjectTemplate< UDatasmithStaticMeshTemplate >( StaticMesh ) : nullptr;

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET_ACCESSORS( LightMapCoordinateIndex, StaticMesh, PreviousStaticMeshTemplate );

	DATASMITHOBJECTTEMPLATE_CONDITIONALSET_ACCESSORS( LightMapResolution, StaticMesh, PreviousStaticMeshTemplate );

	// Section info map
	bool bResetSectionInfoMap = false;
	if ( PreviousStaticMeshTemplate )
	{
		// If the number of sections is different, their order might be different (eg. from mesh editing) so the SectionInfoMap must be reset
		bResetSectionInfoMap = PreviousStaticMeshTemplate->SectionInfoMap.Map.Num() != StaticMesh->GetSectionInfoMap().Map.Num();
	}

	SectionInfoMap.Apply(&StaticMesh->GetSectionInfoMap(), !bResetSectionInfoMap && PreviousStaticMeshTemplate ? &PreviousStaticMeshTemplate->SectionInfoMap : nullptr);

	// Build settings
	for ( int32 SourceModelIndex = 0; SourceModelIndex < BuildSettings.Num(); ++SourceModelIndex )
	{
		if ( !StaticMesh->IsSourceModelValid( SourceModelIndex ) )
		{
			continue;
		}

		FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel( SourceModelIndex );
		
		FDatasmithMeshBuildSettingsTemplate* PreviousBuildSettingsTemplate = nullptr;
		
		if ( PreviousStaticMeshTemplate && PreviousStaticMeshTemplate->BuildSettings.IsValidIndex( SourceModelIndex ) )
		{
			PreviousBuildSettingsTemplate = &PreviousStaticMeshTemplate->BuildSettings[ SourceModelIndex ];
		}

		BuildSettings[ SourceModelIndex ].Apply( &SourceModel.BuildSettings, PreviousBuildSettingsTemplate );
	}

	// Materials
	for ( int32 StaticMaterialIndex = 0; StaticMaterialIndex < StaticMaterials.Num(); ++StaticMaterialIndex )
	{
		// If the SectionInfoMap was reset, the materials must be reapplied to follow it
		bool bIgnorePreviousTemplate = bResetSectionInfoMap;
		if ( !StaticMesh->GetStaticMaterials().IsValidIndex( StaticMaterialIndex ) )
		{
			// If it's a new added material, the value must be force applied to it by ignoring the previous template
			StaticMesh->GetStaticMaterials().AddDefaulted();
			bIgnorePreviousTemplate = true;
		}

		FStaticMaterial& StaticMaterial = StaticMesh->GetStaticMaterials()[ StaticMaterialIndex ];

		FDatasmithStaticMaterialTemplate* PreviousStaticMaterialTemplate = nullptr;

		if ( !bIgnorePreviousTemplate && PreviousStaticMeshTemplate && PreviousStaticMeshTemplate->StaticMaterials.IsValidIndex( StaticMaterialIndex ) )
		{
			PreviousStaticMaterialTemplate = &PreviousStaticMeshTemplate->StaticMaterials[ StaticMaterialIndex ];
		}

		StaticMaterials[ StaticMaterialIndex ].Apply( &StaticMaterial, PreviousStaticMaterialTemplate );
	}

	if ( PreviousStaticMeshTemplate )
	{
		// Remove materials that aren't in the template anymore
		for ( int32 MaterialIndexToRemove = PreviousStaticMeshTemplate->StaticMaterials.Num() - 1; MaterialIndexToRemove >= StaticMaterials.Num(); --MaterialIndexToRemove )
		{
			if ( StaticMesh->GetStaticMaterials().IsValidIndex( MaterialIndexToRemove ) )
			{
				StaticMesh->GetStaticMaterials().RemoveAt( MaterialIndexToRemove );
			}
		}
	}

	// Make sure that the StaticMaterials are in the same order than the StaticMeshLODResources.Sections will be after the StaticMesh is built (see BuildVertexBuffer in StaticMeshBuilder.cpp)
	const int32 NumLODs = StaticMesh->GetNumSourceModels();
	for ( int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex )
	{
		const FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription( LODIndex );
		if ( MeshDescription && MeshDescription->PolygonGroups().Num() == StaticMesh->GetStaticMaterials().Num() )
		{
			FStaticMeshConstAttributes Attributes(*MeshDescription);
			TArray< FStaticMaterial > TempStaticMaterials;
			TPolygonGroupAttributesConstRef< FName > PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
			int32 SectionIndex = 0;
			for ( const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs() )
			{
				int32 MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName( PolygonGroupImportedMaterialSlotNames[ PolygonGroupID ] );
				if ( MaterialIndex == INDEX_NONE )
				{
					MaterialIndex = PolygonGroupID.GetValue();
				}
				TempStaticMaterials.Add( StaticMesh->GetStaticMaterials()[ MaterialIndex ] );

				// Note that the StaticMesh.SectionInfoMap MaterialIndex will overwrite the StaticMeshLODResources.Sections MaterialIndex through FStaticMeshRenderData::ResolveSectionInfo()
				// This ensures there won't be any mismatch when that happens
				FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get( LODIndex, SectionIndex );
				SectionInfo.MaterialIndex = SectionIndex;
				StaticMesh->GetSectionInfoMap().Set( LODIndex, SectionIndex, SectionInfo );

				++SectionIndex;
			}

			// Set the StaticMaterials with respect to LOD 0
			if ( LODIndex == 0 )
			{
				StaticMesh->SetStaticMaterials(MoveTemp( TempStaticMaterials ));
			}
		}
	}
#endif // #if WITH_EDITORONLY_DATA

	return Destination;
}

void UDatasmithStaticMeshTemplate::Load( const UObject* Source )
{
#if WITH_EDITORONLY_DATA
	const UStaticMesh* SourceStaticMesh = Cast< UStaticMesh >( Source );

	if ( !SourceStaticMesh )
	{
		return;
	}

	LightMapCoordinateIndex = SourceStaticMesh->GetLightMapCoordinateIndex();
	LightMapResolution = SourceStaticMesh->GetLightMapResolution();

	// Section info map
	SectionInfoMap.Load( SourceStaticMesh->GetSectionInfoMap() );

	// Build settings
	BuildSettings.Empty( SourceStaticMesh->GetNumSourceModels() );

	for ( const FStaticMeshSourceModel& SourceModel : SourceStaticMesh->GetSourceModels() )
	{
		FDatasmithMeshBuildSettingsTemplate BuildSettingsTemplate;
		BuildSettingsTemplate.Load( SourceModel.BuildSettings );

		BuildSettings.Add( MoveTemp( BuildSettingsTemplate ) );
	}

	// Materials
	StaticMaterials.Empty( SourceStaticMesh->GetStaticMaterials().Num() );

	for ( const FStaticMaterial& StaticMaterial : SourceStaticMesh->GetStaticMaterials())
	{
		FDatasmithStaticMaterialTemplate StaticMaterialTemplate;
		StaticMaterialTemplate.Load( StaticMaterial );

		StaticMaterials.Add( MoveTemp( StaticMaterialTemplate ) );
	}
#endif // #if WITH_EDITORONLY_DATA
}

bool UDatasmithStaticMeshTemplate::Equals( const UDatasmithObjectTemplate* Other ) const
{
	const UDatasmithStaticMeshTemplate* TypedOther = Cast< UDatasmithStaticMeshTemplate >( Other );

	if ( !TypedOther )
	{
		return false;
	}

	bool bEquals = LightMapCoordinateIndex == TypedOther->LightMapCoordinateIndex;
	bEquals = bEquals && ( LightMapResolution == TypedOther->LightMapResolution );

	bEquals = bEquals && SectionInfoMap.Equals( TypedOther->SectionInfoMap );

	// Build settings
	bEquals = bEquals && ( BuildSettings.Num() == TypedOther->BuildSettings.Num() );

	if ( bEquals )
	{
		for ( int32 BuildSettingIndex = 0; BuildSettingIndex < BuildSettings.Num(); ++BuildSettingIndex )
		{
			bEquals = bEquals && BuildSettings[ BuildSettingIndex ].Equals( TypedOther->BuildSettings[ BuildSettingIndex ] );

			if ( !bEquals )
			{
				return false;
			}
		}
	}

	// Materials
	bEquals = bEquals && ( StaticMaterials.Num() == TypedOther->StaticMaterials.Num() );

	if ( bEquals )
	{
		for ( int32 StaticMaterialIndex = 0; StaticMaterialIndex < StaticMaterials.Num(); ++StaticMaterialIndex )
		{
			bEquals = bEquals && StaticMaterials[ StaticMaterialIndex ].Equals( TypedOther->StaticMaterials[ StaticMaterialIndex ] );

			if ( !bEquals )
			{
				return false;
			}
		}
	}

	return bEquals;
}
