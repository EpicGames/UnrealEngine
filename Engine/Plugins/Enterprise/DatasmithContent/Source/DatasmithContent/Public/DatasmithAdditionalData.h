// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Templates/Requires.h"
#include "Templates/SubclassOf.h"
#include "AssetRegistry/AssetData.h"
#include "DatasmithAssetImportData.h"

#include <type_traits>

#include "DatasmithAdditionalData.generated.h"

#define UE_API DATASMITHCONTENT_API

/**
 * Base class for all additional data storable on datasmith assets.
 *
 * @usage:
 * Translator can push custom data on assets in order to exploit that data latter (@see UDatasmithCustomActionBase)
 */
UCLASS(MinimalAPI)
class UDatasmithAdditionalData : public UObject
{
	GENERATED_BODY()

public:
	UE_API UDatasmithAdditionalData();

};

namespace Datasmith
{
	/**
	 * Create an instance of UDatasmithAdditionalData, used to quickly extend data stored on datasmith assets.
	 *
	 * @tparam DataType     Type of additional data to create. Must inherit from UDatasmithAdditionalData.
	 * @return DataType*    Created instance
	 */
	template<
		class DataType
		UE_REQUIRES(std::is_convertible_v<DataType*, UDatasmithAdditionalData*>)
	>
	inline DataType* MakeAdditionalData()
	{
		UClass* Class = DataType::StaticClass();
		UObject* Outer = (UObject*)GetTransientPackage();
		EObjectFlags Flags = RF_NoFlags; // EObjectFlags::RF_Transactional;

		return NewObject<DataType>(Outer, Class, NAME_None, Flags);
	}

	/**
	 * Fetch all additional data of the specified type for the given asset
	 *
	 * @tparam DataType             Subclass of UDatasmithAdditionalData
	 * @param SourceAssetData       Asset on which the additional data is fetched
	 * @param MaxCount              Optional limitation of the result length
	 * @return TArray<DataType*>    The list of additional data stored on the given asset
	 */
	template<
		class DataType
		UE_REQUIRES(std::is_convertible_v<DataType*, UDatasmithAdditionalData*>)
	>
	inline TArray<DataType*> GetMultipleAdditionalData(const FAssetData& SourceAssetData, int MaxCount=0)
	{
		TArray<DataType*> Result;

#if WITH_EDITOR
		UObject* Asset = SourceAssetData.GetAsset();
		if (UDatasmithAssetImportData* ImportData = Cast<UDatasmithAssetImportData>(Datasmith::GetAssetImportData(Asset)))
		{
			for (UDatasmithAdditionalData* Data : ImportData->AdditionalData)
			{
				if (DataType* TypedData = Cast<DataType>(Data))
				{
					Result.Add(TypedData);
					if (MaxCount && Result.Num() >= MaxCount)
					{
						break;
					}
				}
			}
		}
#endif // WITH_EDITOR

		return Result;
	}

	/**
	 * Fetch an additional data on an asset
	 *
	 * @tparam DataType         subclass of UDatasmithAdditionalData
	 * @param SourceAssetData   Asset on which the additional data is fetched
	 * @return DataType*        nullptr if no Additional data is stored on the given asset
	 */
	template<
		class DataType
		UE_REQUIRES(std::is_convertible_v<DataType*, UDatasmithAdditionalData*>)
	>
	inline DataType* GetAdditionalData(const FAssetData& SourceAssetData)
	{
		TArray<DataType*> Result = GetMultipleAdditionalData<DataType>(SourceAssetData, 1);
		return Result.Num() ? Result[0] : nullptr;
	}
}

#undef UE_API
