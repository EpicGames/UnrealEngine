// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Scene.h"
#include "GameFramework/Actor.h"

#include "DatasmithAreaLightActor.generated.h"

// Keep in sync with EDatasmithLightShape from DatasmithDefinitions
UENUM(BlueprintType)
enum class EDatasmithAreaLightActorShape : uint8
{
	Rectangle,
	Disc,
	Sphere,
	Cylinder,
	None
};

UENUM(BlueprintType)
enum class EDatasmithAreaLightActorType : uint8
{
	Point,
	Spot,
	IES_DEPRECATED, // Added for consistency with type EDatasmithAreaLightType. 
					// Removing this will result in 'EDatasmithAreaLightActorType MAX' enum value in the editor.
	Rect,
};

UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class ADatasmithAreaLightActor : public AActor
{
	GENERATED_BODY()

public:
	UPROPERTY(Category = Mobility, EditAnywhere, BlueprintReadOnly)
	TEnumAsByte<EComponentMobility::Type> Mobility;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	EDatasmithAreaLightActorType LightType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	EDatasmithAreaLightActorShape LightShape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (EditCondition = "LightShape != EDatasmithAreaLightActorShape::None"))
	FVector2D Dimensions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	float Intensity;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	ELightUnits IntensityUnits;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	FLinearColor Color;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	float Temperature;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	TObjectPtr<class UTextureLightProfile> IESTexture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta=(DisplayName = "Use IES Intensity"))
	bool bUseIESBrightness;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta=(DisplayName = "IES Intensity Scale"))
	float IESBrightnessScale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	FRotator Rotation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	float SourceRadius;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light")
	float SourceLength;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta=(UIMin = "8.0", UIMax = "16384.0", SliderExponent = "5.0"))
	float AttenuationRadius;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (UIMin = "1.0", UIMax = "80.0", SliderExponent = "1.0"))
	float SpotlightInnerAngle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light", meta = (UIMin = "0.0", UIMax = "80.0", SliderExponent = "1.0"))
	float SpotlightOuterAngle;

public:
	ADatasmithAreaLightActor();
};