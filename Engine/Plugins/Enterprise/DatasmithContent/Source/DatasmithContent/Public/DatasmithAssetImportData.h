// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportOptions.h"

#include "EditorFramework/AssetImportData.h"

#include "DatasmithAssetImportData.generated.h"

#define UE_API DATASMITHCONTENT_API

/**
 * Structure that fill the same role as FAssetImportInfo, but for SourceUri.
 * Eventually, the SourceUri should be directly added to FAssetImportInfo and replace the "RelativeFilename".
 */
USTRUCT()
struct FDatasmithImportInfo
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	FDatasmithImportInfo() {}


	explicit FDatasmithImportInfo(const FString& InSourceUri, FString InSourceHashString = FString())
		: SourceUri(InSourceUri)
		, SourceHash(InSourceHashString)
	{}

	FDatasmithImportInfo(const FString& InSourceUri, FMD5Hash InSourceHash)
		: SourceUri(InSourceUri)
		, SourceHash(LexToString(InSourceHash))
	{}

	/** The Uri of to the source that this asset was imported from. */
	UPROPERTY()
	FString SourceUri;

	/**
	 * The MD5 hash of the source when it was imported. Should be updated alongside the SourceUri
	 */
	UPROPERTY()
	FString SourceHash;

	UE_API void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const;
	UE_DEPRECATED(5.4, "Call the version that takes FAssetRegistryTagsContext instead.")
	UE_API void GetAssetRegistryTags(TArray<UObject::FAssetRegistryTag>& OutTags) const;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI)
class UDatasmithAssetImportData : public UAssetImportData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(BlueprintReadWrite, Category = Asset, meta = (ShowOnlyInnerProperties))
	FDatasmithAssetImportOptions AssetImportOptions;

	UPROPERTY(EditAnywhere, Instanced, Category = Asset, meta = (ShowOnlyInnerProperties))
	TArray<TObjectPtr<class UDatasmithAdditionalData>> AdditionalData;

	UPROPERTY(EditAnywhere, Category = "External Source")
	FDatasmithImportInfo DatasmithImportInfo;
#endif		// WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI)
class UDatasmithStaticMeshImportData : public UDatasmithAssetImportData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = StaticMesh, meta = (ShowOnlyInnerProperties))
	FDatasmithStaticMeshImportOptions ImportOptions;

public:
	typedef TPair< FDatasmithStaticMeshImportOptions, FDatasmithAssetImportOptions > DefaultOptionsPair;
	static UE_API UDatasmithStaticMeshImportData* GetImportDataForStaticMesh( UStaticMesh* StaticMesh, TOptional< DefaultOptionsPair > DefaultImportOptions = TOptional< DefaultOptionsPair >() );

#endif		// WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, HideCategories = (InternalProperty))
class UE_DEPRECATED(5.8, "The class UDatasmithStaticMeshCADImportData has been deprecated and will be removed next release. Please use UDatasmithParametricSurfaceData and child classes instead.")
UDatasmithStaticMeshCADImportData : public UDatasmithStaticMeshImportData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = StaticMesh, meta = (ShowOnlyInnerProperties))
	FDatasmithTessellationOptions TessellationOptions;

	UPROPERTY(VisibleDefaultsOnly, Category = InternalProperty)
	double	ModelUnit;

	UPROPERTY(VisibleDefaultsOnly, Category = InternalProperty)
	double	ModelTolerance;

protected:
	UPROPERTY(VisibleDefaultsOnly, Category = InternalProperty)
	FString	ResourcePath;

	UPROPERTY(VisibleDefaultsOnly, Category = InternalProperty)
	FString	ResourceFilename;

	UPROPERTY(VisibleDefaultsOnly, Category = InternalProperty)
	TArray<FString>	AuxiliaryFilenames;

public:
	typedef TTuple<FDatasmithTessellationOptions, FDatasmithStaticMeshImportOptions, FDatasmithAssetImportOptions> DefaultOptionsTuple;
	static UE_API UDatasmithStaticMeshCADImportData* GetCADImportDataForStaticMesh( UStaticMesh* StaticMesh, TOptional< DefaultOptionsTuple > DefaultImportOptions = TOptional< DefaultOptionsTuple >() );

	UE_API void SetResourcePath(const FString& FilePath);
	UE_API const FString& GetResourcePath();
#endif		// WITH_EDITORONLY_DATA

protected:
	/** Overridden serialize function to read in/write out the unexposed data */
	UE_API virtual void Serialize(FArchive& Ar) override;
};

/**
 * Base class for import data and options used when importing any asset from Datasmith
 */
UCLASS(MinimalAPI)
class UDatasmithSceneImportData : public UAssetImportData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = "Options", meta = (ShowOnlyInnerProperties))
	FDatasmithImportBaseOptions BaseOptions;

	UPROPERTY(EditAnywhere, Category = "External Source")
	FDatasmithImportInfo DatasmithImportInfo;

	//~ UObject interface
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif //WITH_EDITOR
	//~ End UObject interface

#endif // WITH_EDITORONLY_DATA
};

/**
 * Import data and options specific to Datasmith scenes imported through the translator system
 */
UCLASS(MinimalAPI)
class UDatasmithTranslatedSceneImportData : public UDatasmithSceneImportData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = "Options", meta = (ShowOnlyInnerProperties))
	TArray<TObjectPtr<UDatasmithOptionsBase>> AdditionalOptions;
#endif // WITH_EDITORONLY_DATA
};

/**
 * Import data and options specific to tessellated Datasmith scenes
 */
UCLASS(MinimalAPI)
class UDatasmithCADImportSceneData : public UDatasmithSceneImportData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category = "Tessellation", meta = (ShowOnlyInnerProperties))
	FDatasmithTessellationOptions TessellationOptions;
#endif		// WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI)
class UDatasmithMDLSceneImportData : public UDatasmithSceneImportData
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithGLTFSceneImportData : public UDatasmithSceneImportData
{
	GENERATED_BODY()
 public:
	UPROPERTY(VisibleAnywhere, Category = "Scene Info", DisplayName="Generator Name")
	FString Generator;

	UPROPERTY(VisibleAnywhere, Category = "Scene Info", DisplayName="Version")
	float Version;

	UPROPERTY(VisibleAnywhere, Category = "Scene Info", DisplayName="Author")
	FString Author;

	UPROPERTY(VisibleAnywhere, Category = "Scene Info", DisplayName="License")
	FString License;

	UPROPERTY(VisibleAnywhere, Category = "Scene Info", DisplayName="Source")
	FString Source;
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithStaticMeshGLTFImportData : public UDatasmithStaticMeshImportData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = InternalProperty)
	FString SourceMeshName;
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithFBXSceneImportData : public UDatasmithSceneImportData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bGenerateLightmapUVs;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString TexturesDir;

	// Corresponds to a EDatasmithFBXIntermediateSerializationType
	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	uint8 IntermediateSerialization;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bColorizeMaterials;
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithDeltaGenAssetImportData : public UDatasmithAssetImportData
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithDeltaGenSceneImportData : public UDatasmithFBXSceneImportData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bMergeNodes;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bOptimizeDuplicatedNodes;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bRemoveInvisibleNodes;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bSimplifyNodeHierarchy;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportVar;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString VarPath;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportPos;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString PosPath;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportTml;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString TmlPath;
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithVREDAssetImportData : public UDatasmithAssetImportData
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, EditInlineNew)
class UDatasmithVREDSceneImportData : public UDatasmithFBXSceneImportData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bMergeNodes;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bOptimizeDuplicatedNodes;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportMats;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString MatsPath;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportVar;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bCleanVar;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString VarPath;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportLightInfo;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString LightInfoPath;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	bool bImportClipInfo;

	UPROPERTY(VisibleAnywhere, Category = ImportOptions)
	FString ClipInfoPath;
};

namespace Datasmith
{
	DATASMITHCONTENT_API UAssetImportData* GetAssetImportData(UObject* Asset);
}

#undef UE_API
