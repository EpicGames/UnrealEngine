// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "Templates/SubclassOf.h"

#include "DatasmithAssetUserData.generated.h"

#define UE_API DATASMITHCONTENT_API

class UDatasmithObjectTemplate;

/** Asset user data that can be used with Datasmith on Actors and other objects  */
UCLASS(MinimalAPI, BlueprintType, meta = (ScriptName = "DatasmithUserData", DisplayName = "Datasmith User Data"))
class UDatasmithAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:

	// Meta-data are available at runtime in game, i.e. used in blueprint to display build-boarded information
	typedef TMap<FName, FString> FMetaDataContainer;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Datasmith User Data", meta = (ScriptName = "Metadata", DisplayName = "Metadata"))
	TMap<FName, FString> MetaData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> > ObjectTemplates;

	UE_API virtual bool IsPostLoadThreadSafe() const override;
	UE_API virtual void PostLoad() override;
#endif

	static UE_API FString GetDatasmithUserDataValueForKey(UObject* Object, FName Key, bool bPartialMatchKey = false);
	static UE_API TArray<FString> GetDatasmithUserDataValuesForKey(UObject* Object, FName Key, bool bPartialMatchKey = false);
	static UE_API UDatasmithAssetUserData* GetDatasmithUserData(UObject* Object);
	static UE_API bool SetDatasmithUserDataValueForKey(UObject* Object, FName Key, const FString & Value);

	// Meta data keys for Datasmith objects
	static UE_API const TCHAR* UniqueIdMetaDataKey;
};

#undef UE_API
