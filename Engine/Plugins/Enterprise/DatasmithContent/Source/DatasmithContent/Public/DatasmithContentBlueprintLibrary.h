// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"

#include "DatasmithContentBlueprintLibrary.generated.h"

#define UE_API DATASMITHCONTENT_API

class UDatasmithAssetUserData;

UCLASS(MinimalAPI, meta = (ScriptName = "DatasmithContentLibrary"))
class UDatasmithContentBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Get the Datasmith User Data of a given object
	 * @param	Object	The Object from which to retrieve the Datasmith User Data.
	 * @return			The Datasmith User Data if it exists; nullptr, otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith User Data")
	static UE_API UDatasmithAssetUserData* GetDatasmithUserData(UObject* Object);

	/**
	 * Get the value of the given key for the Datasmith User Data of the given object.
	 * @param	Object				The Object from which to retrieve the Datasmith User Data.
	 * @param	Key					The key to find in the Datasmith User Data.
	 * @return						The string value associated with the given key
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith User Data")
	static UE_API FString GetDatasmithUserDataValueForKey(UObject* Object, FName Key, bool bPartialMatchKey = false);

	/**
	 * Get the values of the given key for the Datasmith User Data of the given object.
	 * @param	Object				The Object from which to retrieve the Datasmith User Data.
	 * @param	Key					The key to find in the Datasmith User Data.
	 * @param	bPartialMatchKey	If true, check for contains, rather than exact match.
	 * @return						The string value associated with the given key
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith User Data")
	static UE_API TArray<FString> GetDatasmithUserDataValuesForKey(UObject* Object, FName Key, bool bPartialMatchKey = false);

	/**
	 * Get the keys and values for which the associated value contains the string to match for the Datasmith User Data of the given object.
	 * @param	Object			The Object from which to retrieve the Datasmith User Data.
	 * @param	StringToMatch	The string to match in the values.
	 * @param	OutKeys			Output array of keys for which the associated values contain the string to match.
	 * @param	OutValues		Output array of values associated to the keys.
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith User Data")
	static UE_API void GetDatasmithUserDataKeysAndValuesForValue(UObject* Object, const FString& StringToMatch, TArray<FName>& OutKeys, TArray<FString>& OutValues);

#if WITH_EDITOR

	/**
	 *	Find all Datasmith User Data of loaded objects of the given type.
	 *	This is a slow operation, so editor only.
	 *	@param	ObjectClass		Class of the object on which to filter, if specificed; otherwise there's no filtering
	 *	@param	OutUserData		Output array of Datasmith User Data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith User Data")
	static UE_API void GetAllDatasmithUserData(TSubclassOf<UObject> ObjectClass, TArray<UDatasmithAssetUserData*>& OutUserData);

	/**
	 *	Find all loaded objects of the given type that have a Datasmith User Data that contains the given key and their associated values.
	 *	This is a slow operation, so editor only.
	 *	@param	Key			The key to find in the Datasmith User Data.
	 *	@param	ObjectClass	Class of the object on which to filter, if specificed; otherwise there's no filtering
	 *	@param	OutObjects	Output array of objects for which the Datasmith User Data match the given key.
	 *	@param	OutValues	Output array of values associated with each object in OutObjects.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Datasmith User Data", meta = (DeterminesOutputType = "ObjectClass", DynamicOutputParam = "OutObjects"))
	static UE_API void GetAllObjectsAndValuesForKey(FName Key, TSubclassOf<UObject> ObjectClass, TArray<UObject*>& OutObjects, TArray<FString>& OutValues);

#endif // WITH_EDITOR
};

#undef UE_API
