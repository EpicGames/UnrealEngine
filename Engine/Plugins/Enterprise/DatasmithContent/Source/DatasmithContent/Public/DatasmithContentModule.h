// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()
#include "Logging/LogMacros.h"

#define DATASMITHCONTENT_MODULE_NAME TEXT("DatasmithContent")

DECLARE_LOG_CATEGORY_EXTERN( LogDatasmithContent, Log, All );

/**
 * The public interface of the DatasmithContent module
 */
class IDatasmithContentModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to IDatasmithContent
	 *
	 * @return Returns DatasmithContent singleton instance, loading the module on demand if needed
	 */
	static inline IDatasmithContentModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDatasmithContentModule>(DATASMITHCONTENT_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATASMITHCONTENT_MODULE_NAME);
	}

	/**
	 * @returns temporary directory created for this process
	 */
	UE_DEPRECATED(5.8, "IDatasmithContentModule::GetTempDir will be removed in next release.")
	virtual const FString& GetTempDir() const = 0;
};

