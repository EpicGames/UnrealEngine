// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"
#include "AssetRegistry/AssetData.h"
#include "DatasmithCustomAction.generated.h"

#define UE_API DATASMITHCONTENT_API


/**
 * Interface of a CustomAction.
 * This is a way to quickly expose some custom processing.
 */
class IDatasmithCustomAction
{
public:
	virtual ~IDatasmithCustomAction() = default;

	/** @return Displayed name of the action */
	virtual const FText& GetLabel() = 0;

	/** @return Displayed tooltip of the action */
	virtual const FText& GetTooltip() = 0;

	/**
	 * Called by Datasmith when we displays potential action for some assets
	 * (eg. when the context menu of the content browser is used).
	 *
	 * @param SelectedAssets: Set of candidate assets for processing
	 * @return true when this action is applicable on the given set
	 */
	virtual bool CanApplyOnAssets(const TArray<FAssetData>& SelectedAssets) = 0;

	/**
	 * The actual processing, called by Datasmith when the user trigger an action.
	 * @param SelectedAssets: Set of candidate assets for processing
	 */
	virtual void ApplyOnAssets(const TArray<FAssetData>& SelectedAssets) = 0;

	/**
	 * Called by Datasmith when we displays potential action for some actors
	 * (eg. when the context menu of the level editor is used).
	 *
	 * @param SelectedActors: Set of candidate actors for processing
	 * @return true when this action is applicable on the given set
	 */
	virtual bool CanApplyOnActors(const TArray<AActor*>& SelectedActors) = 0;

	/**
	 * The actual processing, called by Datasmith when the user trigger an action.
	 * @param SelectedActors: Set of candidate actors for processing
	 */
	virtual void ApplyOnActors(const TArray<AActor*>& SelectedActors) = 0;
};


/**
 * Base class for actions available to the end-user through the Content browser contextual menu.
 * By extending this class, the custom action is automatically registered to be available for the end user
 */
UCLASS(MinimalAPI, Abstract)
class UDatasmithCustomActionBase : public UObject, public IDatasmithCustomAction
{
	GENERATED_BODY()

public:
	virtual const FText& GetLabel() override PURE_VIRTUAL( GetLabel, return FText::GetEmpty(); )
	virtual const FText& GetTooltip() override PURE_VIRTUAL( GetTooltip, return FText::GetEmpty(); )

	virtual bool CanApplyOnAssets(const TArray<FAssetData>&) { return false; }
	virtual void ApplyOnAssets(const TArray<FAssetData>&) {}

	virtual bool CanApplyOnActors(const TArray<AActor*>&) { return false; }
	virtual void ApplyOnActors(const TArray<AActor*>&) {}
};


class FDatasmithCustomActionManager
{
public:
	UE_API FDatasmithCustomActionManager();

	UE_API TArray<UDatasmithCustomActionBase*> GetApplicableActions(const TArray<FAssetData>& SelectedAssets);
	UE_API TArray<UDatasmithCustomActionBase*> GetApplicableActions(const TArray<AActor*>& SelectedActors);

private:
	TArray<TStrongObjectPtr<UDatasmithCustomActionBase>> RegisteredActions;
};


#undef UE_API
