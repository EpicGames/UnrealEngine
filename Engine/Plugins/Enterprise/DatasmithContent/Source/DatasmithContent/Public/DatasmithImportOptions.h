// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"

#include "DatasmithImportOptions.generated.h"

#define UE_API DATASMITHCONTENT_API

class FJsonObject;

UENUM()
enum class EDatasmithImportSearchPackagePolicy : uint8
{
	/** Search only in current package */
	Current UMETA(DisplayName = "Current", DisplayValue = "Current"),

	/** Search in all packages */
	All,
};

UENUM()
enum class EDatasmithImportAssetConflictPolicy : uint8
{
	/** Replace existing asset with new one */
	Replace,

	/** Update existing asset with new values */
	Update,

	/** Use existing asset instead of creating new one */
	Use,

	/** Skip new asset */
	Ignore,
};

UENUM()
enum class EDatasmithImportActorPolicy : uint8
{
	/** Import new actors, update and delete existing actors. Doesn't recreate actors that exist in the source both not in the destination. */
	Update,

	/** Same as update but recreates deleted actors so that the source and destination are the same. */
	Full,

	/** Skip importing a certain type of actors */
	Ignore,
};

UENUM()
enum class EDatasmithImportMaterialQuality : uint8
{
	UseNoFresnelCurves,

	UseSimplifierFresnelCurves,

	UseRealFresnelCurves,
};

UENUM()
enum class EDatasmithImportLightmapMin : uint8
{
	LIGHTMAP_16		UMETA(DisplayName = "16"),
	LIGHTMAP_32		UMETA(DisplayName = "32"),
	LIGHTMAP_64		UMETA(DisplayName = "64"),
	LIGHTMAP_128	UMETA(DisplayName = "128"),
	LIGHTMAP_256	UMETA(DisplayName = "256"),
	LIGHTMAP_512	UMETA(DisplayName = "512"),
};

UENUM()
enum class EDatasmithImportLightmapMax : uint8
{
	LIGHTMAP_64		UMETA(DisplayName = "64"),
	LIGHTMAP_128	UMETA(DisplayName = "128"),
	LIGHTMAP_256	UMETA(DisplayName = "256"),
	LIGHTMAP_512	UMETA(DisplayName = "512"),
	LIGHTMAP_1024	UMETA(DisplayName = "1024"),
	LIGHTMAP_2048	UMETA(DisplayName = "2048"),
	LIGHTMAP_4096	UMETA(DisplayName = "4096")
};

UENUM()
enum class EDatasmithImportScene : uint8
{
	NewLevel		UMETA(DisplayName = "Create New Level", ToolTip = "Create a new Level and spawn the actors after the import."),

	CurrentLevel	UMETA(DisplayName = "Merge to Current Level", ToolTip = "Use the current Level to spawn the actors after the import."),

	AssetsOnly		UMETA(DisplayName = "Assets Only", ToolTip = "Do not modify the Level after import. No actor will be created (including the Blueprint if requested by the ImportHierarchy"),
};

UENUM()
enum class EDatasmithCADStitchingTechnique : uint8
{
	StitchingNone = 0,
	StitchingHeal,
	StitchingSew,
};

UENUM()
enum class EDatasmithCADRetessellationRule : uint8
{
	All = 0,
	SkipDeletedSurfaces,
};

USTRUCT(BlueprintType)
struct FDatasmithAssetImportOptions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName PackagePath;
};

USTRUCT(BlueprintType)
struct FDatasmithStaticMeshImportOptions
{
	GENERATED_USTRUCT_BODY()

	UE_API FDatasmithStaticMeshImportOptions();

	/** Minimum resolution for auto-generated lightmap UVs */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Lightmap)
	EDatasmithImportLightmapMin MinLightmapResolution;

	/** Maximum resolution for auto-generated lightmap UVs */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Lightmap)
	EDatasmithImportLightmapMax MaxLightmapResolution;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Lightmap, meta = (DisplayName = "Generate Lightmap UVs"))
	bool bGenerateLightmapUVs;

	UPROPERTY(BlueprintReadWrite, Category = Mesh, Transient)
	bool bRemoveDegenerates;

public:
	static UE_API int32 ConvertLightmapEnumToValue( EDatasmithImportLightmapMin EnumValue );
	static UE_API int32 ConvertLightmapEnumToValue( EDatasmithImportLightmapMax EnumValue );

	bool operator == (const FDatasmithStaticMeshImportOptions& Other) const
	{
		return MinLightmapResolution == Other.MinLightmapResolution
			&& MaxLightmapResolution == Other.MaxLightmapResolution
			&& bGenerateLightmapUVs == Other.bGenerateLightmapUVs
			&& bRemoveDegenerates == Other.bRemoveDegenerates;
	}

	bool operator != (const FDatasmithStaticMeshImportOptions& Other) const {
		return !operator==(Other);
	}
};

USTRUCT(BlueprintType)
struct FDatasmithReimportOptions
{
	GENERATED_BODY()

public:
	UE_API FDatasmithReimportOptions();

	/** Specifies whether or not to update Datasmith Scene Actors in the current Level */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SyncCurrentLevelActors", meta = (DisplayName = "Datasmith Scene Actors"))
	bool bUpdateActors;

	/** Specifies whether or not to add back Actors you've deleted from the current Level */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SyncCurrentLevelActors", meta = (DisplayName = "Re-Spawn Deleted Actors", EditCondition = "bUpdateActors"))
	bool bRespawnDeletedActors;
};

USTRUCT(BlueprintType)
struct FDatasmithImportBaseOptions
{
	GENERATED_USTRUCT_BODY()

	UE_API FDatasmithImportBaseOptions();

	/** Specifies where to put the content */
	UPROPERTY(BlueprintReadWrite, Category = Import, Transient)
	EDatasmithImportScene SceneHandling; // Not displayed, not saved

	/** Specifies whether or not to import geometry */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Process, meta = (DisplayName = "Geometry"))
	bool bIncludeGeometry;

	/** Specifies whether or not to import materials and textures */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Process, meta = (DisplayName = "Materials & Textures"))
	bool bIncludeMaterial;

	/** Specifies whether or not to import lights */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Process, meta = (DisplayName = "Lights"))
	bool bIncludeLight;

	/** Specifies whether or not to import cameras */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Process, meta = (DisplayName = "Cameras"))
	bool bIncludeCamera;

	/** Specifies whether or not to import animations */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Process, meta = (DisplayName = "Animations"))
	bool bIncludeAnimation;

	UPROPERTY(BlueprintReadWrite, AdvancedDisplay, Category = Process, meta = (ShowOnlyInnerProperties))
	FDatasmithAssetImportOptions AssetOptions;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Process, meta = (ShowOnlyInnerProperties))
	FDatasmithStaticMeshImportOptions StaticMeshOptions;

	bool CanIncludeAnimation() const { return bIncludeGeometry || bIncludeCamera || bIncludeLight; }
};

namespace UE::DatasmithTessellation
{
const double MinTessellationAngle = 5;
const double MinTessellationEdgeLength = 1.;
const double MinTessellationChord = 0.005;  // Usual value in CAD software is 0.02 cm
}

USTRUCT(BlueprintType)
struct FDatasmithTessellationOptions
{
	GENERATED_BODY()

	FDatasmithTessellationOptions(float InChordTolerance = 0.2f, float InMaxEdgeLength = 0.0f, float InNormalTolerance = 20.0f, EDatasmithCADStitchingTechnique InStitchingTechnique = EDatasmithCADStitchingTechnique::StitchingSew)
		: ChordTolerance(InChordTolerance)
		, MaxEdgeLength(InMaxEdgeLength)
		, NormalTolerance(InNormalTolerance)
		, StitchingTechnique(InStitchingTechnique)
	{
	}

	/**
	 * Maximum distance between any point on a triangle generated by the tessellation process and the actual surface.
	 * The lower the value the more triangles.
	 * Default value is 0.2, minimal value is 0.005 cm.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry & Tessellation Options", meta = (Units = cm, ToolTip = "Maximum distance between any generated triangle and the original surface. Smaller values make more triangles.", ClampMin = "0.005"))
	float ChordTolerance;

	/**
	 * Maximum length of edges of triangles generated by the tessellation process.
	 * The length is in scene/model unit. The smaller the more triangles are generated.
	 * Value of 0 means no constraint on length of edges
	 * Default value is 0 to disable this criteria, and 1. cm is its minimal value if enable.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = cm, DisplayName = "Max Edge Length", ToolTip = "Maximum length of any edge in the generated triangles. Smaller values make more triangles.", ClampMin = "0.0"))
	float MaxEdgeLength;

	/**
	 * Maximum angle between the normal of two triangles generated by the tessellation process.
	 * The angle is expressed in degree. The smaller the more triangles are generated.
	 * Default value is 20 degrees, min value is 5 degrees.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = deg, ToolTip = "Maximum angle between adjacent triangles. Smaller values make more triangles.", ClampMin = "5.0", ClampMax = "90.0"))
	float NormalTolerance;


	/**
	 * Stitching technique applied on neighbouring surfaces before tessellation.
	 * None : No stitching applied. This is the default.
	 * Sewing : Connects surfaces which physically share a boundary but not topologically within a set of objects.
	 *          This technique can modify the structure of the model by removing and adding objects.
	 * Healing : Connects surfaces which physically share a boundary but not topologically within an object.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (ToolTip = "Stitching technique applied on model before tessellation. Sewing could impact number of objects."))
	EDatasmithCADStitchingTechnique StitchingTechnique;

	bool bUseCADKernel = false;

protected:
	/**
	 * Tolerance used to determine if a surface should be tessellate or not.
	 * Any surface which is narrower than the geometric tolerance
	 * in one of the iso direction will not be tessellated
	 * Value is in centimeter
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = cm, ToolTip = "Tolerance used to determine if a surface should be tessellated or not."))
	double GeometricTolerance = 0.001;

	/**
	 * Tolerance used to determine if two surfaces should be stitched.
	 */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Geometry & Tessellation Options", meta = (Units = cm, ToolTip = "Tolerance used to determine if two surfaces should be stitched.", editCondition = "StitchingTechnique!=EDatasmithCADStitchingTechnique::StitchingNone"))
	double StitchingTolerance = 0.001;

public:
	bool operator == (const FDatasmithTessellationOptions& Other) const
	{
		return FMath::IsNearlyEqual(ChordTolerance, Other.ChordTolerance)
			&& FMath::IsNearlyEqual(MaxEdgeLength, Other.MaxEdgeLength)
			&& FMath::IsNearlyEqual(NormalTolerance, Other.NormalTolerance)
			&& StitchingTechnique == Other.StitchingTechnique
			&& FMath::IsNearlyEqual(GeometricTolerance, Other.GeometricTolerance)
			&& FMath::IsNearlyEqual(StitchingTolerance, Other.StitchingTolerance);
	}

	uint32 GetHash() const
	{
		uint32 Hash = uint32(StitchingTechnique);
		for (float Param : {ChordTolerance, MaxEdgeLength, NormalTolerance})
		{
			Hash = HashCombine(Hash, GetTypeHash(Param));
		}
		return Hash;
	}
	/** Helper functions to get geometrical values in the right unit, cm (native) or mm */
	double GetGeometricTolerance( bool bInMillimeter = false ) const { return bInMillimeter ? GeometricTolerance * 10. : GeometricTolerance; }
	double GetStitchingTolerance( bool bInMillimeter = false ) const { return bInMillimeter ? StitchingTolerance * 10. : StitchingTolerance; }
};

USTRUCT(BlueprintType)
struct FDatasmithRetessellationOptions : public FDatasmithTessellationOptions
{
	GENERATED_BODY()

	FDatasmithRetessellationOptions(float InChordTolerance = 0.2f, float InMaxEdgeLength = 0.0f, float InNormalTolerance = 20.0f, EDatasmithCADStitchingTechnique InStitchingTechnique = EDatasmithCADStitchingTechnique::StitchingSew, EDatasmithCADRetessellationRule InRetessellationRule = EDatasmithCADRetessellationRule::All)
		: FDatasmithTessellationOptions(InChordTolerance, InMaxEdgeLength, InNormalTolerance, InStitchingTechnique)
		, RetessellationRule(InRetessellationRule)
	{
	}

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Retessellation Options", meta = (ToolTip = "Regenerate deleted surfaces during retesselate or ignore them"))
	EDatasmithCADRetessellationRule RetessellationRule = EDatasmithCADRetessellationRule::All;

public:
	void operator = (const FDatasmithTessellationOptions& Other)
	{
		ChordTolerance = Other.ChordTolerance;
		MaxEdgeLength = Other.MaxEdgeLength;
		NormalTolerance = Other.NormalTolerance;
		StitchingTechnique = Other.StitchingTechnique;
		GeometricTolerance = Other.GetGeometricTolerance();
		StitchingTolerance = Other.GetStitchingTolerance();
	}
};

/**
 * Base class for all import options in datasmith.
 *
 * Notable feature: forces a full serialization of its properties (by opposition
 * to the standard delta serialization which stores only the diff wrt the CDO)
 * The intent is to store the exact options used in a previous import.
 */
UCLASS(MinimalAPI)
class UDatasmithOptionsBase : public UObject
{
	GENERATED_BODY()

public:
	UE_API void Serialize(FStructuredArchive::FRecord Record) override;
};


UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings)
class UDatasmithCommonTessellationOptions : public UDatasmithOptionsBase
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry & Tessellation Options", meta = (ShowOnlyInnerProperties))
	FDatasmithTessellationOptions Options;
};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings, HideCategories = ("NotVisible"))
class UDatasmithImportOptions : public UDatasmithOptionsBase
{
	GENERATED_UCLASS_BODY()

public:
	/** Specifies where to search for assets */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportSearchPackagePolicy SearchPackagePolicy; // Not displayed. Kept for future use

	/** Specifies what to do when material conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportAssetConflictPolicy MaterialConflictPolicy; // Not displayed. Kept for future use

	/** Specifies what to do when texture conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportAssetConflictPolicy TextureConflictPolicy; // Not displayed. Kept for future use

	/** Specifies what to do when actor conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportActorPolicy StaticMeshActorImportPolicy; // Not displayed. Kept for future use

	/** Specifies what to do when light conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportActorPolicy LightImportPolicy; // Not displayed. Kept for future use

	/** Specifies what to do when material conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportActorPolicy CameraImportPolicy; // Not displayed. Kept for future use

	/** Specifies what to do when actor conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportActorPolicy OtherActorImportPolicy; // Not displayed. Kept for future use

	/** Specifies what to do when material conflicts */
	UPROPERTY(Transient, AdvancedDisplay)
	EDatasmithImportMaterialQuality MaterialQuality; // Not displayed. Kept for future use

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Options", meta = (ShowOnlyInnerProperties))
	FDatasmithImportBaseOptions BaseOptions;

	/** Options specific to the reimport process */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Reimport", meta = (ShowOnlyInnerProperties))
	FDatasmithReimportOptions ReimportOptions;

	/** Name of the imported file without its path */
	UPROPERTY(BlueprintReadWrite, Category = "NotVisible")
	FString FileName;

	/** Full path of the imported file */
	UPROPERTY(BlueprintReadWrite, Category = "NotVisible")
	FString FilePath;

	UPROPERTY(BlueprintReadWrite, Category = "NotVisible")
	FString SourceUri;

	/** The hash of the source referenced by SourceUri */
	FMD5Hash SourceHash;

	/** Whether to use or not the same options when loading multiple files. Default false */
	bool bUseSameOptions;

	UE_API void UpdateNotDisplayedConfig( bool bIsAReimport );

	//~ UObject interface
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif //WITH_EDITOR
	//~ End UObject interface
};

#undef UE_API
