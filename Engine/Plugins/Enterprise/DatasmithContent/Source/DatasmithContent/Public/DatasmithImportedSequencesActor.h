// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "DatasmithImportedSequencesActor.generated.h"

#define UE_API DATASMITHCONTENT_API

class ULevelSequence;

UCLASS(MinimalAPI)
class ADatasmithImportedSequencesActor : public AActor
{
public:

	GENERATED_BODY()

	UE_API ADatasmithImportedSequencesActor(const FObjectInitializer& Init);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ImportedSequences")
	TArray<TObjectPtr<ULevelSequence>> ImportedSequences;

    UFUNCTION(BlueprintCallable, Category="ImportedSequences")
	UE_API void PlayLevelSequence(ULevelSequence* SequenceToPlay);
};

#undef UE_API
