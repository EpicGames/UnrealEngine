// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportOptions.h"
#include "Misc/Guid.h"
#include "Serialization/BulkData.h"
#include "Interfaces/Interface_AssetUserData.h"

#include "DatasmithScene.generated.h"

#define UE_API DATASMITHCONTENT_API

class ULevelSequence;
class ULevelVariantSets;
class UMaterialFunction;
class UMaterialInterface;
class UStaticMesh;
class UTexture;
class UWorld;

UCLASS(MinimalAPI)
class UDatasmithScene : public UObject, public IInterface_AssetUserData
{
	GENERATED_BODY()

public:
	UE_API UDatasmithScene();

	UE_API virtual ~UDatasmithScene();

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this Datasmith scene */
	UPROPERTY(EditAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<class UDatasmithSceneImportData> AssetImportData;

	UPROPERTY()
	int32 BulkDataVersion; // Need an external version number because loading of the bulk data is handled externally

	FByteBulkData DatasmithSceneBulkData;

	/** Map of all the static meshes related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< UStaticMesh > > StaticMeshes;

	/** Map of all the cloth related to this Datasmith Scene */
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< UObject > > Clothes; // UChaosClothAsset

	/** Map of all the textures related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< UTexture > > Textures;

	/** Map of all the material functions related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< UMaterialFunction > > MaterialFunctions;

	/** Map of all the materials related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< UMaterialInterface > > Materials;

	/** Map of all the level sequences related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< ULevelSequence > > LevelSequences;

	/** Map of all the level variant sets related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< ULevelVariantSets > > LevelVariantSets;

	/** Array of user data stored with the asset */
	UPROPERTY()
	TArray< TObjectPtr<UAssetUserData> > AssetUserData;
#endif // #if WITH_EDITORONLY_DATA

	//~ Begin UObject Interface
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//~ End UObject Interface

	/** Register the DatasmithScene to the PreWorldRename callback as needed*/
	UE_API void RegisterPreWorldRenameCallback();

	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData( UAssetUserData* InUserData ) override;
	UE_API virtual void RemoveUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass ) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass ) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

#if WITH_EDITOR
private:
	/** Called before a world is renamed */
	void OnPreWorldRename(UWorld* World, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename);

	bool bPreWorldRenameCallbackRegistered;
#endif

public:
	UE_API virtual void Serialize( FArchive& Archive ) override;
};

#undef UE_API
