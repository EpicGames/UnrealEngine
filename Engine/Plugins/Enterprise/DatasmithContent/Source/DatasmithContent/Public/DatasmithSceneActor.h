// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "DatasmithSceneActor.generated.h"

#define UE_API DATASMITHCONTENT_API

class UFactory;

UCLASS(MinimalAPI)
class ADatasmithSceneActor : public AActor
{
	GENERATED_BODY()

public:

	UE_API ADatasmithSceneActor();

	UPROPERTY(VisibleAnywhere, Category="Datasmith")
	TObjectPtr<class UDatasmithScene> Scene;

	/** Map of all the actors related to this Datasmith Scene */
	UPROPERTY(VisibleAnywhere, Category="Datasmith", AdvancedDisplay)
	TMap< FName, TSoftObjectPtr< AActor > > RelatedActors;

	// Clean up delegates before destruction 
	UE_API void BeginDestroy() override;

private:
#if WITH_EDITOR
	// For backward compatibility with UE <= 4.21
	void EnsureDatasmithIdsForRelatedActors();

	// Backward compatibility fix will happen on PostLoad.
	UE_API void PostLoad() override;

	// We need this to monitor when a sub-level is loaded
	void OnMapChange(uint32 MapChangeFlags);

	// Cleans the invalid Soft Object Ptr
	void OnActorDeleted(AActor* ActorDeleted);

	// Reattach newly imported actors if they've been moved / cut and pasted
	void OnAssetPostImport(UFactory* InFactory, UObject* ActorAdded);

	void RegisterDelegates();
	void UnregisterDelegates();

	FDelegateHandle OnActorDeletedDelegateHandle;
#endif // WITH_EDITOR
};

#undef UE_API
