// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithActorTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

// hold template informations common to all AActors.
UCLASS(MinimalAPI)
class UDatasmithActorTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:

	UDatasmithActorTemplate()
		: UDatasmithObjectTemplate(true)
	{}
	  
	/** Layers this actor belongs to. (see AActor::Layers) */
	UPROPERTY()
	TSet<FName> Layers;

	/** Tags on this actor. (see AActor::Tags) */
	UPROPERTY()
	TSet<FName> Tags;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;

	/** Helper function to get the typed Actor from either a component or an actor */
	template< typename T >
	static T* GetActor( UObject* Object )
	{
		return const_cast< T* >( GetActor< T >( static_cast< const UObject* >( Object ) ) );
	}

	template< typename T >
	static const T* GetActor( const UObject* Object )
	{
		const UActorComponent* ActorComponent = Cast< UActorComponent >( Object );
		const UObject* Actor;
		
		if ( ActorComponent )
		{
			Actor = static_cast< const UObject* >( ActorComponent->GetOwner() );
		}
		else
		{
			Actor = Object;
		}

		return Cast< T >( Actor );
	}
};

#undef UE_API
