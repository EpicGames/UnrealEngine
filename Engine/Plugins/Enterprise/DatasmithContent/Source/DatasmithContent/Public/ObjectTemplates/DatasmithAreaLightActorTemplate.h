// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithAreaLightActor.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "Engine/Scene.h"

#include "DatasmithAreaLightActorTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

UCLASS(MinimalAPI)
class UDatasmithAreaLightActorTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	EDatasmithAreaLightActorType LightType;

	UPROPERTY()
	EDatasmithAreaLightActorShape LightShape;

	UPROPERTY()
	FVector2D Dimensions;

	UPROPERTY()
	FLinearColor Color;

	UPROPERTY()
	float Intensity;

	UPROPERTY()
	ELightUnits IntensityUnits;

	UPROPERTY()
	float Temperature;

	UPROPERTY()
	TSoftObjectPtr< class UTextureLightProfile > IESTexture;

	UPROPERTY()
	bool bUseIESBrightness;

	UPROPERTY()
	float IESBrightnessScale;

	UPROPERTY()
	FRotator Rotation;

	UPROPERTY()
	float SourceRadius;

	UPROPERTY()
	float SourceLength;

	UPROPERTY()
	float AttenuationRadius;

public:
	UE_API UDatasmithAreaLightActorTemplate();

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
