// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "CineCameraActor.h"

#include "DatasmithCineCameraActorTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

USTRUCT()
struct FDatasmithCameraLookatTrackingSettingsTemplate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint8 bEnableLookAtTracking : 1;

	UPROPERTY()
	uint8 bAllowRoll : 1;

	UPROPERTY()
	TSoftObjectPtr< AActor > ActorToTrack;

public:
	UE_API FDatasmithCameraLookatTrackingSettingsTemplate();

	UE_API void Apply( FCameraLookatTrackingSettings* Destination, FDatasmithCameraLookatTrackingSettingsTemplate* PreviousTemplate );
	UE_API void Load( const FCameraLookatTrackingSettings& Source );
	UE_API bool Equals( const FDatasmithCameraLookatTrackingSettingsTemplate& Other ) const;
};

UCLASS(MinimalAPI)
class UDatasmithCineCameraActorTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UDatasmithCineCameraActorTemplate()
		: UDatasmithObjectTemplate(true)
	{}

	UPROPERTY()
	FDatasmithCameraLookatTrackingSettingsTemplate LookatTrackingSettings;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
