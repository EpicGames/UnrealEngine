// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineCameraComponent.h"
#include "Engine/Scene.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithCineCameraComponentTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

struct FCameraFilmbackSettings;
struct FCameraFocusSettings;
struct FCameraLensSettings;

USTRUCT()
struct FDatasmithCameraFilmbackSettingsTemplate
{
	GENERATED_BODY()

	UPROPERTY()
	float SensorWidth = 0.0f;

	UPROPERTY()
	float SensorHeight = 0.0f;

	void Apply( FCameraFilmbackSettings* Destination, const FDatasmithCameraFilmbackSettingsTemplate* PreviousTemplate );
	void Load( const FCameraFilmbackSettings& Source );
	bool Equals( const FDatasmithCameraFilmbackSettingsTemplate& Other ) const;
};

USTRUCT()
struct FDatasmithCameraLensSettingsTemplate
{
	GENERATED_BODY()

	UPROPERTY()
	float MaxFStop = 0.0f;

	void Apply( FCameraLensSettings* Destination, const FDatasmithCameraLensSettingsTemplate* PreviousTemplate );
	void Load( const FCameraLensSettings& Source );
	bool Equals( const FDatasmithCameraLensSettingsTemplate& Other ) const;
};

USTRUCT()
struct FDatasmithCameraFocusSettingsTemplate
{
	GENERATED_BODY()

	UPROPERTY()
	ECameraFocusMethod FocusMethod = ECameraFocusMethod::DoNotOverride;

	UPROPERTY()
	float ManualFocusDistance = 0.0f;

	void Apply( FCameraFocusSettings* Destination, const FDatasmithCameraFocusSettingsTemplate* PreviousTemplate );
	void Load( const FCameraFocusSettings& Source );
	bool Equals( const FDatasmithCameraFocusSettingsTemplate& Other ) const;
};

USTRUCT()
struct FDatasmithPostProcessSettingsTemplate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint32 bOverride_WhiteTemp:1;

	UPROPERTY()
	uint32 bOverride_ColorSaturation:1;

	UPROPERTY()
	uint32 bOverride_VignetteIntensity:1;

	UPROPERTY()
	uint32 bOverride_AutoExposureMethod:1;

	UPROPERTY()
	uint32 bOverride_CameraISO:1;

	UPROPERTY()
	uint32 bOverride_CameraShutterSpeed:1;

	UPROPERTY()
	uint8 bOverride_DepthOfFieldFstop:1;

	UPROPERTY()
	float WhiteTemp = 0.0f;

	UPROPERTY()
	float VignetteIntensity = 0.0f;

	UPROPERTY()
	FVector4 ColorSaturation;

	UPROPERTY()
	TEnumAsByte< enum EAutoExposureMethod > AutoExposureMethod;

	UPROPERTY()
	float CameraISO = 0.0f;

	UPROPERTY()
	float CameraShutterSpeed = 0.0f;

	UPROPERTY()
	float DepthOfFieldFstop = 0.0f;

public:
	UE_API FDatasmithPostProcessSettingsTemplate();

	UE_API void Apply( struct FPostProcessSettings* Destination, const FDatasmithPostProcessSettingsTemplate* PreviousTemplate );
	UE_API void Load( const FPostProcessSettings& Source );
	UE_API bool Equals( const FDatasmithPostProcessSettingsTemplate& Other ) const;
};

UCLASS(MinimalAPI)
class UDatasmithCineCameraComponentTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDatasmithCameraFilmbackSettingsTemplate FilmbackSettings;

	UPROPERTY()
	FDatasmithCameraLensSettingsTemplate LensSettings;

	UPROPERTY()
	FDatasmithCameraFocusSettingsTemplate FocusSettings;

	UPROPERTY()
	float CurrentFocalLength = 0.0f;

	UPROPERTY()
	float CurrentAperture = 0.0f;

	UPROPERTY()
	FDatasmithPostProcessSettingsTemplate PostProcessSettings;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
