// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithLandscapeTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

class UMaterialInterface;

UCLASS(MinimalAPI)
class UDatasmithLandscapeTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UDatasmithLandscapeTemplate()
		: UDatasmithObjectTemplate(true)
	{}

	UPROPERTY()
	TObjectPtr<UMaterialInterface> LandscapeMaterial;

	UPROPERTY()
	int32 StaticLightingLOD;

	UE_API virtual UObject* UpdateObject(UObject* Destination, bool bForce = false) override;
	UE_API virtual void Load(const UObject* Source) override;
	UE_API virtual bool Equals(const UDatasmithObjectTemplate* Other) const override;
};

#undef UE_API
