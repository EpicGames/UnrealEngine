// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithLightComponentTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

UCLASS(MinimalAPI)
class UDatasmithLightComponentTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UE_API UDatasmithLightComponentTemplate();

	UPROPERTY()
	uint8 bVisible : 1;

	UPROPERTY()
	uint32 CastShadows : 1;

	UPROPERTY()
	uint32 bUseTemperature : 1;

	UPROPERTY()
	uint32 bUseIESBrightness : 1;

	UPROPERTY()
	float Intensity;

	UPROPERTY()
	float Temperature;

	UPROPERTY()
	float IESBrightnessScale;

	UPROPERTY()
	FLinearColor LightColor;

	UPROPERTY()
	TObjectPtr<class UMaterialInterface> LightFunctionMaterial;

	UPROPERTY()
	TObjectPtr<class UTextureLightProfile> IESTexture;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
