// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"

#include "DatasmithMaterialInstanceTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

struct FStaticParameterSet;
class UMaterialInterface;
class UMaterialInstanceConstant;
class UTexture;

USTRUCT()
struct FDatasmithStaticParameterSetTemplate
{
	GENERATED_BODY();

public:
	UPROPERTY()
	TMap< FName, bool > StaticSwitchParameters;

public:
	UE_API void Apply( UMaterialInstanceConstant* Destination, FDatasmithStaticParameterSetTemplate* PreviousTemplate );
	UE_API void Load( const UMaterialInstanceConstant& Source, bool bOverridesOnly = true );
	UE_API void LoadRebase( const UMaterialInstanceConstant& Source, const FDatasmithStaticParameterSetTemplate& ComparedTemplate, const FDatasmithStaticParameterSetTemplate* MergedTemplate);
	UE_API bool Equals( const FDatasmithStaticParameterSetTemplate& Other ) const;
};

/**
 * Applies material instance data to a material instance if it hasn't changed since the last time we've applied a template.
 * Supports Scalar parameters, Vector parameters, Texture parameters and Static parameters
 */
UCLASS(MinimalAPI)
class UDatasmithMaterialInstanceTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
	
	UPROPERTY()
	TSoftObjectPtr< UMaterialInterface > ParentMaterial;

	UPROPERTY()
	TMap< FName, float > ScalarParameterValues;

	UPROPERTY()
	TMap< FName, FLinearColor > VectorParameterValues;

	UPROPERTY()
	TMap< FName, TSoftObjectPtr< UTexture > > TextureParameterValues;

	UPROPERTY()
	FDatasmithStaticParameterSetTemplate StaticParameters;

protected:
	UE_API virtual void LoadRebase(const UObject* Source, const UDatasmithObjectTemplate* BaseTemplate, bool bMergeTemplate) override;
	UE_API virtual bool HasSameBase(const UDatasmithObjectTemplate* Other) const override;
	/**
	 * Loads all the source object properties into the template, regardless if they are different from the default values or not.
	**/
	UE_API virtual void LoadAll(const UObject* Source);
};

#undef UE_API
