// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DatasmithObjectTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

UCLASS(MinimalAPI, abstract)
class UDatasmithObjectTemplate : public UObject
{
	GENERATED_BODY()

public:

	UDatasmithObjectTemplate()
		: UObject()
		, bIsActorTemplate(false)
	{}


	UDatasmithObjectTemplate(bool bInIsActorTemplate)
		: UObject()
		, bIsActorTemplate(bInIsActorTemplate)
	{}

	/**
	 * Updates the Destination object with the values stored in the object template
	 *
	 * @param Destination	The object to apply this template to
	 * @param bForce		Force the update of the template on all properties, even if they were changed from the previous template values.
	 * @return The UObject on which the object template could be set on. Returns nullptr if the update has failed
	 */
	virtual UObject* UpdateObject(UObject* Destination, bool bForce = false) { return nullptr; }

	/**
	 * Fills this template properties with the values from the Source object.
	 */
	virtual void Load( const UObject* Source ) {}

	/**
	 * Returns if this template equals another template of the same type.
	 */
	virtual bool Equals( const UDatasmithObjectTemplate* Other ) const { return false; }

	/**
	 * Updates the Destination object with the values stored in the object template
	 * If the update is successful, replaces the object template of the Destination object with itself
	 *
	 * @param Destination	The object to apply this template to
	 * @param bForce		Force the update of the template on all properties, even if they were changed from the previous template values.
	 */
	UE_API void Apply( UObject* Destination, bool bForce = false );

	// Is this template for an actor
	const bool bIsActorTemplate = false;

	/**
	 * Returns the difference between the object template of the Destination object and the SourceTemplate
	 * object template
	 *
	 * @param Destination		The object to get the difference from
	 * @param SourceTemplate	The object template to compare to.
	 */
	static UE_API UDatasmithObjectTemplate* GetDifference(UObject* Destination, UDatasmithObjectTemplate* Source);

protected:
	/**
	 * Fills this template properties with the values from the Source object after rebasing (changing its parent) with the base object defined in BaseTemplate.
	 *
	 * @param Source			The object to get the template from.
	 * @param BaseTemplate		The template which is using a different base object. Its base will be used to rebase the template.
	 * @param bMergeTemplate	If True, the loaded template will merge the changes contained in the BaseTemplate as well.
	 */
	virtual void LoadRebase(const UObject* Source, const UDatasmithObjectTemplate* BaseTemplate, bool bLoadDiff = false) { Load(Source); }

	/**
	 * Returns if this template has the same base as another template of the same type, this indicates if their diffs can be safely compared.
	 */
	virtual bool HasSameBase(const UDatasmithObjectTemplate* Other) const { return true; }
};

// Sets Destination->MemberName with the value of MemberName only if PreviousTemplate is null or has the same value for MemberName as the Destination.
// The goal is to set a new value only if it wasn't changed (overriden) in the Destination.
#define DATASMITHOBJECTTEMPLATE_CONDITIONALSET(MemberName, Destination, PreviousTemplate) \
	if ( !PreviousTemplate || Destination->MemberName == PreviousTemplate->MemberName ) \
	{ \
		Destination->MemberName = MemberName; \
	}

#define DATASMITHOBJECTTEMPLATE_CONDITIONALSET_ACCESSORS(MemberName, Destination, PreviousTemplate) \
	if ( !PreviousTemplate || Destination->Get##MemberName() == PreviousTemplate->MemberName ) \
	{ \
		Destination->Set##MemberName(MemberName); \
	}

// Specialized version of DATASMITHOBJECTTEMPLATE_CONDITIONALSET to handle SoftObjectPtr to Ptr assignment
#define DATASMITHOBJECTTEMPLATE_CONDITIONALSETSOFTOBJECTPTR(MemberName, Destination, PreviousTemplate) \
	if ( !PreviousTemplate || Destination->MemberName == PreviousTemplate->MemberName.Get() ) \
	{ \
		Destination->MemberName = MemberName.LoadSynchronous(); \
	}

struct FDatasmithObjectTemplateUtils
{
	static UE_API bool HasObjectTemplates( UObject* Outer );

	static UE_API class UDatasmithAssetUserData* FindOrCreateDatasmithUserData( UObject* Outer );

	static UE_API TMap< TSubclassOf< UDatasmithObjectTemplate >, TObjectPtr<UDatasmithObjectTemplate> >* FindOrCreateObjectTemplates( UObject* Outer );

	static UE_API UDatasmithObjectTemplate* GetObjectTemplate( UObject* Outer, TSubclassOf< UDatasmithObjectTemplate > Subclass );

	template < typename T >
	static T* GetObjectTemplate( UObject* Outer )
	{
		return Cast< T >( GetObjectTemplate( Outer, T::StaticClass() ) );
	}

	static UE_API void SetObjectTemplate( UObject* Outer, UDatasmithObjectTemplate* ObjectTemplate );

	/**
	 * Based on existing data, last import and current import, deduce resulting data that reflects User work.
	 *  - Use values from the new set
	 *  - Keep User-added values
	 *  - Ignore User-removed values
	 *
	 * @param OldSet        Previously imported set
	 * @param CurrentSet    Data after user edition
	 * @param NewSet        Data being imported
	 * @return              Merged data set
	 */
	static UE_API TSet<FName> ThreeWaySetMerge(const TSet<FName>& OldSet, const TSet<FName>& CurrentSet, const TSet<FName>& NewSet);

	/**
	 * Compares two sets for equality. Order has no influence.
	 * @note: LegacyCompareEqual also check for order
	 */
	static UE_API bool SetsEquals(const TSet<FName>& Left, const TSet<FName>& Right);

};

#undef UE_API
