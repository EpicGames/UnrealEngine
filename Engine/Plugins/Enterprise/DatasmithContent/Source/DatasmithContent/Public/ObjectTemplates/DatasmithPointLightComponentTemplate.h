// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Scene.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithPointLightComponentTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

UCLASS(MinimalAPI)
class UDatasmithPointLightComponentTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UE_API UDatasmithPointLightComponentTemplate();

	UPROPERTY()
	ELightUnits IntensityUnits;

	UPROPERTY()
	float SourceRadius;

	UPROPERTY()
	float SourceLength;

	UPROPERTY()
	float AttenuationRadius;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
