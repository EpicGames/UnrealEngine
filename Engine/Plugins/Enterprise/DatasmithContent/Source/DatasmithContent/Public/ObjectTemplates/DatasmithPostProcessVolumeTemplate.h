// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithCineCameraComponentTemplate.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithPostProcessVolumeTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

/**
 * Represents an APostProcessVolume
 */
UCLASS(MinimalAPI)
class UDatasmithPostProcessVolumeTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UDatasmithPostProcessVolumeTemplate()
		: UDatasmithObjectTemplate(true)
	{}

	UPROPERTY()
	FDatasmithPostProcessSettingsTemplate Settings;

	UPROPERTY()
	uint32 bEnabled:1;

	UPROPERTY()
	uint32 bUnbound:1;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
