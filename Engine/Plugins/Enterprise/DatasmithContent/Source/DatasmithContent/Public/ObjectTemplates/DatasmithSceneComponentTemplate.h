// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithSceneComponentTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

UCLASS(MinimalAPI)
class UDatasmithSceneComponentTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FTransform RelativeTransform;

	UPROPERTY()
	TEnumAsByte< EComponentMobility::Type > Mobility;

	UPROPERTY()
	TSoftObjectPtr< USceneComponent > AttachParent;

	UPROPERTY()
	bool bVisible;

	UPROPERTY()
	bool bCastShadow = true;

	UPROPERTY()
	TSet<FName> Tags;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
