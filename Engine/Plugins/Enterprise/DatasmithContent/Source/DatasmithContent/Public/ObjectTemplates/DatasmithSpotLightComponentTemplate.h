// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithObjectTemplate.h"

#include "DatasmithSpotLightComponentTemplate.generated.h"

UCLASS()
class UDatasmithSpotLightComponentTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UDatasmithSpotLightComponentTemplate();

	UPROPERTY()
	float InnerConeAngle;

	UPROPERTY()
	float OuterConeAngle;

	virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	virtual void Load( const UObject* Source ) override;
	virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};
