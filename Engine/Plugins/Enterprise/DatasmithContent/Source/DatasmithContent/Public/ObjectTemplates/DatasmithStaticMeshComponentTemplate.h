// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTemplates/DatasmithSceneComponentTemplate.h"

#include "DatasmithStaticMeshComponentTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

UCLASS(MinimalAPI)
class UDatasmithStaticMeshComponentTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TObjectPtr<class UStaticMesh> StaticMesh;

	UPROPERTY()
	TArray< TObjectPtr<class UMaterialInterface> > OverrideMaterials;

	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;
};

#undef UE_API
