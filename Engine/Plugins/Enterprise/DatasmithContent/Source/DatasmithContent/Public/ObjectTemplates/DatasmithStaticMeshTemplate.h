// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "DatasmithObjectTemplate.h"

#include "DatasmithStaticMeshTemplate.generated.h"

#define UE_API DATASMITHCONTENT_API

struct FMeshSectionInfo;
struct FMeshSectionInfoMap;
struct FStaticMaterial;

USTRUCT()
struct FDatasmithMeshBuildSettingsTemplate
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint8 bUseMikkTSpace:1;

	UPROPERTY()
	uint8 bRecomputeNormals:1;

	UPROPERTY()
	uint8 bRecomputeTangents:1;

	UPROPERTY()
	uint8 bRemoveDegenerates:1;

	UPROPERTY()
	uint8 bUseHighPrecisionTangentBasis:1;

	UPROPERTY()
	uint8 bUseFullPrecisionUVs:1;

	UPROPERTY()
	uint8 bGenerateLightmapUVs:1;

	UPROPERTY()
	int32 MinLightmapResolution;

	UPROPERTY()
	int32 SrcLightmapIndex;

	UPROPERTY()
	int32 DstLightmapIndex;

public:
	UE_API FDatasmithMeshBuildSettingsTemplate();

	UE_API void Apply( FMeshBuildSettings* Destination, FDatasmithMeshBuildSettingsTemplate* PreviousTemplate );
	UE_API void Load( const FMeshBuildSettings& Source );
	UE_API bool Equals( const FDatasmithMeshBuildSettingsTemplate& Other ) const;
};

USTRUCT()
struct FDatasmithStaticMaterialTemplate
{
	GENERATED_BODY()

public:
	UE_API FDatasmithStaticMaterialTemplate();

	UPROPERTY()
	FName MaterialSlotName;

	UPROPERTY()
	TObjectPtr<class UMaterialInterface> MaterialInterface;

	UE_API void Apply( FStaticMaterial* Destination, FDatasmithStaticMaterialTemplate* PreviousTemplate );
	UE_API void Load( const FStaticMaterial& Source );
	UE_API bool Equals( const FDatasmithStaticMaterialTemplate& Other ) const;
};

USTRUCT()
struct FDatasmithMeshSectionInfoTemplate
{
	GENERATED_BODY()

public:
	UE_API FDatasmithMeshSectionInfoTemplate();

	UPROPERTY()
	int32 MaterialIndex;

	UE_API void Apply( FMeshSectionInfo* Destination, FDatasmithMeshSectionInfoTemplate* PreviousTemplate );
	UE_API void Load( const FMeshSectionInfo& Source );
	UE_API bool Equals( const FDatasmithMeshSectionInfoTemplate& Other ) const;
};

USTRUCT()
struct FDatasmithMeshSectionInfoMapTemplate
{
	GENERATED_BODY()

	UPROPERTY()
	TMap< uint32, FDatasmithMeshSectionInfoTemplate > Map;

	UE_API void Apply( FMeshSectionInfoMap* Destination, FDatasmithMeshSectionInfoMapTemplate* PreviousTemplate );
	UE_API void Load( const FMeshSectionInfoMap& Source );
	UE_API bool Equals( const FDatasmithMeshSectionInfoMapTemplate& Other ) const;
};

UCLASS(MinimalAPI)
class UDatasmithStaticMeshTemplate : public UDatasmithObjectTemplate
{
	GENERATED_BODY()

public:
	UE_API virtual UObject* UpdateObject( UObject* Destination, bool bForce = false ) override;
	UE_API virtual void Load( const UObject* Source ) override;
	UE_API virtual bool Equals( const UDatasmithObjectTemplate* Other ) const override;

	UPROPERTY( VisibleAnywhere, Category = StaticMesh )
	FDatasmithMeshSectionInfoMapTemplate SectionInfoMap;

	UPROPERTY( VisibleAnywhere, Category = StaticMesh )
	int32 LightMapCoordinateIndex = INDEX_NONE;

	UPROPERTY( VisibleAnywhere, Category = StaticMesh )
	int32 LightMapResolution = 0;

	UPROPERTY( VisibleAnywhere, Category = StaticMesh )
	TArray< FDatasmithMeshBuildSettingsTemplate > BuildSettings;

	UPROPERTY( VisibleAnywhere, Category = StaticMesh )
	TArray< FDatasmithStaticMaterialTemplate > StaticMaterials;
};

#undef UE_API
