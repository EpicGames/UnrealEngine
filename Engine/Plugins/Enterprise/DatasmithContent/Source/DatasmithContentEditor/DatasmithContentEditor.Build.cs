// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DatasmithContentEditor : ModuleRules
	{
		public DatasmithContentEditor(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DatasmithContent",
					"DesktopPlatform",
					"EditorFramework",
					"Engine",
					"Projects",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore",
					"PropertyEditor",
					"SlateCore",
					"Slate",
					"ToolMenus",
					"DetailCustomizations",
                }
			);
		}
	}
}