// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DatasmithScene.h"

#include "ContentBrowserMenuContexts.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithContentEditorModule.h"
#include "DatasmithContentEditorStyle.h"
#include "DatasmithScene.h"

#include "ToolMenuSection.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DatasmithScene)

#define LOCTEXT_NAMESPACE "AssetTypeActions_DatasmithScene"

FText UAssetDefinition_DatasmithScene::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_DatasmithScene_Name", "Datasmith Scene");
}

TSoftClassPtr<UObject> UAssetDefinition_DatasmithScene::GetAssetClass() const
{
	return UDatasmithScene::StaticClass();
}

EAssetCommandResult UAssetDefinition_DatasmithScene::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FOnCreateDatasmithSceneEditor DatasmithSceneEditorHandler = IDatasmithContentEditorModule::Get().GetDatasmithSceneEditorHandler();

	if (DatasmithSceneEditorHandler.IsBound() == false)
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, OpenArgs.LoadObjects<UObject>());
		return EAssetCommandResult::Handled;
	}

	for (UDatasmithScene* DatasmithScene : OpenArgs.LoadObjects<UDatasmithScene>())
	{
		DatasmithSceneEditorHandler.ExecuteIfBound(EToolkitMode::Standalone, OpenArgs.ToolkitHost, DatasmithScene);
	}

	return EAssetCommandResult::Handled;

}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DatasmithScene::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(NSLOCTEXT("DatasmithContentEditorModule", "DatasmithContentAssetCategory", "Datasmith")) };
	return Categories;
}

namespace MenuExtension_DatasmithScene
{

	void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
	{
		if (InObjects.Num() == 0)
		{
			return;
		}

		FOnCreateDatasmithSceneEditor DatasmithSceneEditorHandler = IDatasmithContentEditorModule::Get().GetDatasmithSceneEditorHandler();

		if (DatasmithSceneEditorHandler.IsBound() == false)
		{
			FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, InObjects);
			return;
		}

		for (UObject* Object : InObjects)
		{
			UDatasmithScene* DatasmithScene = Cast<UDatasmithScene>(Object);
			if (DatasmithScene != nullptr)
			{
				DatasmithSceneEditorHandler.ExecuteIfBound(EToolkitMode::Standalone, EditWithinLevelEditor, DatasmithScene);
			}
		}
	}

	void ExecuteToggleDirectLinkAutoReimport(const TArray<TWeakObjectPtr<UDatasmithScene>>& Scenes, bool bEnabled)
	{
		const IDatasmithContentEditorModule& DatasmithContentEditorModule = IDatasmithContentEditorModule::Get();

		for (const TWeakObjectPtr<class UDatasmithScene>& Scene : Scenes)
		{
			if (Scene.IsValid())
			{
				DatasmithContentEditorModule.SetAssetAutoReimport(Scene.Get(), bEnabled);
			}
		}
	}

	void FilterByDirectLinkAutoReimportSupport(TArray<UDatasmithScene*>& Scenes)
	{
		const IDatasmithContentEditorModule& DatasmithContentEditorModule = IDatasmithContentEditorModule::Get();

		for (int32 SceneIndex = Scenes.Num() - 1; SceneIndex >= 0; --SceneIndex)
		{
			const bool bIsReimportAvailable = DatasmithContentEditorModule.IsAssetAutoReimportAvailable(Scenes[SceneIndex]).Get(false);
			if (bIsReimportAvailable)
			{
				continue;
			}
			Scenes.RemoveAtSwap(SceneIndex);
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UDatasmithScene::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					// todo: investigate if it's possible to check AutoReimport availability/state without loading the DatasmithScene asset
					// so that loading would be done only when action(ExecuteToggleDirectLinkAutoReimport) is executed
					TArray<UDatasmithScene*> Scenes = CBContext->LoadSelectedObjects<UDatasmithScene>();

					FilterByDirectLinkAutoReimportSupport(Scenes);

					const bool bHasCanExecuteAutoReimport = Scenes.Num() > 0;
					
					bool bHasAutoReimportEnabled = false;
					const IDatasmithContentEditorModule& DatasmithContentEditorModule = IDatasmithContentEditorModule::Get();
					for (UDatasmithScene* Scene : Scenes)
					{
						bHasAutoReimportEnabled = DatasmithContentEditorModule.IsAssetAutoReimportEnabled(Scene).Get(false);
						if (bHasAutoReimportEnabled)
						{
							break;
						}
					}

					const TAttribute<FText> Label = LOCTEXT("DatasmithScene_ToggleDirectLinkAutoReimport", "Direct Link Auto-Reimport");
					const TAttribute<FText> ToolTip = LOCTEXT("DatasmithScene_ToggleDirectLinkAutoReimportTooltip", "Toggle Direct Link Auto-Reimport for all selected Datasmith Scenes.");
					const FSlateIcon Icon = FSlateIcon(FDatasmithContentEditorStyle::GetStyleSetName(), "DatasmithContent.AutoReimportGrayscale");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([Scenes = TArray<TWeakObjectPtr<UDatasmithScene>>(Scenes), bHasAutoReimportEnabled](const FToolMenuContext& Context) { ExecuteToggleDirectLinkAutoReimport(Scenes, !bHasAutoReimportEnabled); });
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([bHasCanExecuteAutoReimport](const FToolMenuContext& Context) { return bHasCanExecuteAutoReimport; });
					UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([bHasAutoReimportEnabled](const FToolMenuContext& Context) { return bHasAutoReimportEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; });
					
					InSection.AddMenuEntry("DatasmithScene_ToggleDirectLinkAutoReimport", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::Check);

				}
			}));
		}));
	});
}


#undef LOCTEXT_NAMESPACE
