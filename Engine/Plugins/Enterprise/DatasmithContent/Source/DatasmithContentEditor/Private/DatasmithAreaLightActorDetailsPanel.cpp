// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithAreaLightActorDetailsPanel.h"

#include "DatasmithContentEditorModule.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "UObject/UnrealType.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "IDetailsView.h"

#include "ScopedTransaction.h"

#include "Customizations/MobilityCustomization.h"


#define LOCTEXT_NAMESPACE "DatasmithAreaLightActorDetails"


FDatasmithAreaLightActorDetailsPanel::FDatasmithAreaLightActorDetailsPanel()
{
}

TSharedRef<IDetailCustomization> FDatasmithAreaLightActorDetailsPanel::MakeInstance()
{
	return MakeShared< FDatasmithAreaLightActorDetailsPanel >();
}

void FDatasmithAreaLightActorDetailsPanel::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& TransformCategory = DetailBuilder.EditCategory("TransformCommon", LOCTEXT("TransformCommonCategory", "Transform"), ECategoryPriority::Transform);
	TSharedPtr<IPropertyHandle> MobilityHandle = DetailBuilder.GetProperty("Mobility");

	uint8 RestrictedMobilityBits = 0u;
	TransformCategory.AddCustomBuilder(MakeShared<FMobilityCustomization>(MobilityHandle, RestrictedMobilityBits, true /* bForLight */));
}

#undef LOCTEXT_NAMESPACE
