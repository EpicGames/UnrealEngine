// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithContentEditorModule.h"

#include "DatasmithAreaLightActorDetailsPanel.h"
#include "DatasmithContentEditorStyle.h"
#include "DatasmithImportInfoCustomization.h"
#include "DatasmithSceneActorDetailsPanel.h"
#include "DatasmithSceneDetails.h"

#include "AssetToolsModule.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithScene.h"
#include "Engine/Blueprint.h"
#include "IAssetTools.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "DatasmithContentEditorModule"

/**
 * DatasmithContent module implementation (private)
 */
class FDatasmithContentEditorModule : public IDatasmithContentEditorModule
{
public:
	virtual void StartupModule() override
	{
		// Register the details customizer
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
		PropertyModule.RegisterCustomClassLayout( TEXT("DatasmithSceneActor"), FOnGetDetailCustomizationInstance::CreateStatic( &FDatasmithSceneActorDetailsPanel::MakeInstance ) );
		PropertyModule.RegisterCustomClassLayout( TEXT("DatasmithAreaLightActor"), FOnGetDetailCustomizationInstance::CreateStatic(&FDatasmithAreaLightActorDetailsPanel::MakeInstance));

		FDatasmithContentEditorStyle::Initialize();

		RegisterDetailCustomization();
	}

	virtual void ShutdownModule() override
	{
		// Unregister the details customization
		if ( FModuleManager::Get().IsModuleLoaded( TEXT("PropertyEditor") ) )
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked< FPropertyEditorModule >( TEXT("PropertyEditor") );
			PropertyModule.UnregisterCustomClassLayout( TEXT("DatasmithSceneActor") );
			PropertyModule.NotifyCustomizationModuleChanged();
		}

		// Shutdown style set associated with datasmith content
		FDatasmithContentEditorStyle::Shutdown();

		UnregisterDetailCustomization();
	}

	virtual void RegisterSpawnDatasmithSceneActorsHandler( FOnSpawnDatasmithSceneActors InSpawnActorsDelegate ) override
	{
		SpawnActorsDelegate = InSpawnActorsDelegate;
	}

	virtual void UnregisterSpawnDatasmithSceneActorsHandler( FOnSpawnDatasmithSceneActors InSpawnActorsDelegate ) override
	{
		SpawnActorsDelegate.Unbind();
	}

	virtual FOnSpawnDatasmithSceneActors GetSpawnDatasmithSceneActorsHandler() const override
	{
		return SpawnActorsDelegate;
	}

	void RegisterDatasmithSceneEditorHandler(FOnCreateDatasmithSceneEditor InCreateDatasmithSceneEditorDelegate)
	{
		CreateDatasmithSceneEditorDelegate = InCreateDatasmithSceneEditorDelegate;
	}

	virtual void UnregisterDatasmithSceneEditorHandler(FOnCreateDatasmithSceneEditor InCreateDatasmithSceneEditor)
	{
		if (CreateDatasmithSceneEditorDelegate.IsBound() && InCreateDatasmithSceneEditor.GetHandle() == CreateDatasmithSceneEditorDelegate.GetHandle())
		{
			CreateDatasmithSceneEditorDelegate.Unbind();
		}
	}

	virtual FOnCreateDatasmithSceneEditor GetDatasmithSceneEditorHandler() const
	{
		return CreateDatasmithSceneEditorDelegate;
	}

	virtual void RegisterDatasmithImporter(const void* Registrar, const FImporterDescription& ImporterDescription) override
	{
		DatasmithImporterMap.Add(Registrar) = ImporterDescription;
	}

	virtual void UnregisterDatasmithImporter(const void* Registrar) override
	{
		DatasmithImporterMap.Remove(Registrar);
	}

	TArray<FImporterDescription> GetDatasmithImporters()
	{
		TArray<FImporterDescription> Result;

		for (const auto& ImporterDescriptionEntry : DatasmithImporterMap)
		{
			Result.Add(ImporterDescriptionEntry.Value);
		}

		return Result;
	}

	virtual void RegisterSetAssetAutoReimportHandler(FOnSetAssetAutoReimport&& SetAssetAutoReimportDelegate) override
	{
		SetAssetAutoReimportHandler = MoveTemp(SetAssetAutoReimportDelegate);
	}
	
	virtual void UnregisterSetAssetAutoReimportHandler(FDelegateHandle InHandle) override
	{
		if (SetAssetAutoReimportHandler.GetHandle() == InHandle)
		{
			SetAssetAutoReimportHandler.Unbind();
		}
	}
		
	virtual TOptional<bool> SetAssetAutoReimport(UObject* Asset, bool bEnabled) const override
	{
		if (SetAssetAutoReimportHandler.IsBound())
		{
			return SetAssetAutoReimportHandler.Execute(Asset, bEnabled);
		}

		return TOptional<bool>();
	}

	virtual void RegisterIsAssetAutoReimportAvailableHandler(FOnIsAssetAutoReimportAvailable&& IsAssetAutoReimportAvailableDelegate)
	{
		IsAssetAutoReimportAvailableHandler = MoveTemp(IsAssetAutoReimportAvailableDelegate);
	}
	
	virtual void UnregisterIsAssetAutoReimportAvailableHandler(FDelegateHandle InHandle)
	{
		if (IsAssetAutoReimportAvailableHandler.GetHandle() == InHandle)
		{
			IsAssetAutoReimportAvailableHandler.Unbind();
		}
	}
	
	virtual TOptional<bool> IsAssetAutoReimportAvailable(UObject* Asset) const
	{
		if (IsAssetAutoReimportAvailableHandler.IsBound())
		{
			return IsAssetAutoReimportAvailableHandler.Execute(Asset);
		}

		return TOptional<bool>();
	}

	virtual void RegisterIsAssetAutoReimportEnabledHandler(FOnIsAssetAutoReimportEnabled&& IsAssetAutoReimportEnabledDelegate) override
	{
		IsAssetAutoReimportEnabledHandler = MoveTemp(IsAssetAutoReimportEnabledDelegate);
	}
	
	virtual void UnregisterIsAssetAutoReimportEnabledHandler(FDelegateHandle InHandle) override
	{
		if (IsAssetAutoReimportEnabledHandler.GetHandle() == InHandle)
		{
			IsAssetAutoReimportEnabledHandler.Unbind();
		}
	}
	
	virtual TOptional<bool> IsAssetAutoReimportEnabled(UObject* Asset) const override
	{
		if (IsAssetAutoReimportEnabledHandler.IsBound())
		{
			return IsAssetAutoReimportEnabledHandler.Execute(Asset);
		}

		return TOptional<bool>();
	}

	virtual void RegisterBrowseExternalSourceUriHandler(FOnBrowseExternalSourceUri&& BrowseExternalSourceUriDelegate) override
	{
		BrowseExternalSourceUriHandler = MoveTemp(BrowseExternalSourceUriDelegate);
	}

	virtual void UnregisterBrowseExternalSourceUriHandler(FDelegateHandle InHandle) override
	{
		if (BrowseExternalSourceUriHandler.GetHandle() == InHandle)
		{
			BrowseExternalSourceUriHandler.Unbind();
		}
	}

	virtual bool BrowseExternalSourceUri(FName UriScheme, const FString& DefaultUri, FString& OutSourceUri, FString& OutFallbackFilepath) const override
	{
		if (IsAssetAutoReimportEnabledHandler.IsBound())
		{
			return BrowseExternalSourceUriHandler.Execute(UriScheme, DefaultUri, OutSourceUri, OutFallbackFilepath);
		}

		return false;
	}

	virtual void RegisterGetSupportedUriSchemeHandler(FOnGetSupportedUriSchemes&& GetSupportedUriSchemeDelegate) override
	{
		GetSupportedUriSchemeHandler = MoveTemp(GetSupportedUriSchemeDelegate);
	}

	virtual void UnregisterGetSupportedUriSchemeHandler(FDelegateHandle InHandle) override
	{
		if (GetSupportedUriSchemeHandler.GetHandle() == InHandle)
		{
			GetSupportedUriSchemeHandler.Unbind();
		}
	}

	virtual TOptional<TArray<FName>> GetSupportedUriScheme() const override
	{
		if (GetSupportedUriSchemeHandler.IsBound())
		{
			return GetSupportedUriSchemeHandler.Execute();
		}

		return TOptional<TArray<FName>>();
	}

private:
	static TSharedPtr<IDataprepImporterInterface> CreateEmptyDatasmithImportHandler()
	{
		return TSharedPtr<IDataprepImporterInterface>();
	}

	void RegisterDetailCustomization()
	{
		const FName PropertyEditor("PropertyEditor");

		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
		PropertyModule.RegisterCustomPropertyTypeLayout(FDatasmithImportInfo::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDatasmithImportInfoCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(UDatasmithScene::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FDatasmithSceneDetails::MakeDetails));
	}

	void UnregisterDetailCustomization()
	{
		const FName PropertyEditor("PropertyEditor");

		if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded(PropertyEditor))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
			PropertyModule.UnregisterCustomPropertyTypeLayout(FDatasmithImportInfo::StaticStruct()->GetFName());
			PropertyModule.UnregisterCustomClassLayout(UDatasmithScene::StaticClass()->GetFName());
		}
	}

private:
	FOnSpawnDatasmithSceneActors SpawnActorsDelegate;
	FOnCreateDatasmithSceneEditor CreateDatasmithSceneEditorDelegate;
	TMap<const void*, FImporterDescription> DatasmithImporterMap;

	FOnSetAssetAutoReimport SetAssetAutoReimportHandler;
	FOnIsAssetAutoReimportAvailable IsAssetAutoReimportAvailableHandler;
	FOnIsAssetAutoReimportEnabled IsAssetAutoReimportEnabledHandler;
	FOnBrowseExternalSourceUri BrowseExternalSourceUriHandler;
	FOnGetSupportedUriSchemes GetSupportedUriSchemeHandler;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDatasmithContentEditorModule, DatasmithContentEditor);
