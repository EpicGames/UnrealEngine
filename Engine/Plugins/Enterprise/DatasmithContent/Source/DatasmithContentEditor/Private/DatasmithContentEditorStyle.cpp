// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithContentEditorStyle.h"

#include "DatasmithContentModule.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FDatasmithContentEditorStyle::InContent( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(FDatasmithContentEditorStyle::InContent(RelativePath, TEXT(".svg")), __VA_ARGS__)

TSharedPtr<FSlateStyleSet> FDatasmithContentEditorStyle::StyleSet;

void FDatasmithContentEditorStyle::Initialize()
{
	using namespace CoreStyleConstants;

	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("DatasmithContent.AutoReimportGrayscale", new IMAGE_PLUGIN_BRUSH_SVG(TEXT("Icons/DataSmithAutoReimport_16"), Icon16x16));

	// Copy the base style, and change the rounding of the left-side borders so that it can be combined with a "right" widget.
	FButtonStyle ButtonLeftStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>(TEXT("Button"));
	ButtonLeftStyle.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, FVector4(4.0f, 0.0f, 0.0f, 4.0f), FStyleColors::Input, InputFocusThickness))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(4.0f, 0.0f, 0.0f, 4.0f), FStyleColors::Input, InputFocusThickness))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(4.0f, 0.0f, 0.0f, 4.0f), FStyleColors::Input, InputFocusThickness))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0f, 0.0f, 0.0f, 4.0f), FStyleColors::Recessed, InputFocusThickness));

	StyleSet->Set("DatasmithDataprepEditor.ButtonLeft", ButtonLeftStyle);

	// Copy the base style and tweak the padding and right-side border rounding so that it can be combined with a "left" widget.
	FComboBoxStyle ComboRightStyle = FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("SimpleComboBox"));
	FComboButtonStyle& ComboButtonStyle = ComboRightStyle.ComboButtonStyle;
	ComboButtonStyle.SetDownArrowPadding(1);
	FButtonStyle& SimpleButtonStyle = ComboButtonStyle.ButtonStyle;
	SimpleButtonStyle
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, FVector4(0.0f, 4.0f, 4.0f, 0.0f), FStyleColors::Input, InputFocusThickness))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(0.0f, 4.0f, 4.0f, 0.0f), FStyleColors::Input, InputFocusThickness))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(0.0f, 4.0f, 4.0f, 0.0f), FStyleColors::Input, InputFocusThickness))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f), FStyleColors::Recessed, InputFocusThickness));

	StyleSet->Set("DatasmithDataprepEditor.SimpleComboBoxRight", ComboRightStyle);

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FDatasmithContentEditorStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FDatasmithContentEditorStyle::GetStyleSetName()
{
	static FName StyleName("DatasmithContentEditorStyle");
	return StyleName;
}

FString FDatasmithContentEditorStyle::InContent(const FString& RelativePath, const TCHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(DATASMITHCONTENT_MODULE_NAME)->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

#undef IMAGE_PLUGIN_BRUSH_SVG
#undef IMAGE_PLUGIN_BRUSH
