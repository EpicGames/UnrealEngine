// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithImportInfoCustomization.h"

#include "DatasmithAssetImportData.h"
#include "DatasmithContentEditorModule.h"
#include "DatasmithContentEditorStyle.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DatasmithImportInfoCustomization"

FDatasmithImportInfoCustomization::FDatasmithImportInfoCustomization()
{
	if (TOptional<TArray<FName>> SupportedSchemes = IDatasmithContentEditorModule::Get().GetSupportedUriScheme())
	{
		AvailableUriSchemes = MoveTemp(SupportedSchemes.GetValue());
	}
}

TSharedRef<IPropertyTypeCustomization> FDatasmithImportInfoCustomization::MakeInstance()
{
	return MakeShareable(new FDatasmithImportInfoCustomization);
}

void FDatasmithImportInfoCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	if (GetImportInfo() != nullptr)
	{
		const FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();
		const FText SourceUriLabel = LOCTEXT("SourceUri", "Source URI");

		const FButtonStyle* ButtonStyle = &FDatasmithContentEditorStyle::Get()->GetWidgetStyle<FButtonStyle>(TEXT("DatasmithDataprepEditor.ButtonLeft"));
		const FComboBoxStyle* ComboBoxStyle = &FDatasmithContentEditorStyle::Get()->GetWidgetStyle<FComboBoxStyle>(TEXT("DatasmithDataprepEditor.SimpleComboBoxRight"));

		ChildBuilder.AddCustomRow(SourceUriLabel)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(SourceUriLabel)
			.Font(Font)
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.MaxDesiredWidth(TOptional<float>())
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.f)
			[
				SNew(SEditableText)
				.IsReadOnly(true)
				.Text(this, &FDatasmithImportInfoCustomization::GetUriText)
				.ToolTipText(this, &FDatasmithImportInfoCustomization::GetUriText)
				.Font(Font)
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(ButtonStyle)
				.OnClicked(this, &FDatasmithImportInfoCustomization::OnBrowseSourceClicked)
				.ToolTipText(LOCTEXT("ChangePath_Tooltip", "Browse for a new source."))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("...", "..."))
					.Font(Font)
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SAssignNew(SchemeComboBox, SComboBox<FName>)
				.OptionsSource(&AvailableUriSchemes)
				.OnSelectionChanged(this, &FDatasmithImportInfoCustomization::OnSchemeComboBoxChanged)
				.OnGenerateWidget_Lambda([](FName InItem) { return SNew(STextBlock).Text(FText::FromName(InItem)); })
				.ComboBoxStyle(ComboBoxStyle)
				[
					// We only need to display the "down arrow" but SComboBox does not accept SNullWidget as valid content.
					// Use an empty STextBlock instead.
					SNew(STextBlock)
				]
			]
		];
	}
}

FName FDatasmithImportInfoCustomization::GetCurrentScheme() const
{
	FName CurrentScheme;

	if (FDatasmithImportInfo* Info = GetImportInfo())
	{
		// #ueent_todo Once source URI become part of the core API, update this to FSourceUri::GetScheme();
		int32 SchemeSeparatorIndex = Info->SourceUri.Find("://");
		if (SchemeSeparatorIndex != INDEX_NONE)
		{
			CurrentScheme = FName(Info->SourceUri.Left(SchemeSeparatorIndex));
		}
	}

	if (CurrentScheme.IsNone())
	{
		// we should fallback on file selection if we can't determine the scheme.
		CurrentScheme = FName(TEXT("file"));
	}

	return CurrentScheme;
}

FText FDatasmithImportInfoCustomization::GetUriText() const
{
	FDatasmithImportInfo* Info = GetImportInfo();
	if (Info)
	{
		return FText::FromString(Info->SourceUri);
	}
	return LOCTEXT("NoUriFound", "No Source Uri Set");
}

FDatasmithImportInfo* FDatasmithImportInfoCustomization::GetImportInfo() const
{
	TArray<FDatasmithImportInfo*> AssetImportInfo;

	if (PropertyHandle->IsValidHandle())
	{
		PropertyHandle->AccessRawData(reinterpret_cast<TArray<void*>&>(AssetImportInfo));
	}

	if (AssetImportInfo.Num() == 1)
	{
		return AssetImportInfo[0];
	}
	return nullptr;
}

UObject* FDatasmithImportInfoCustomization::GetOuterClass() const
{
	static TArray<UObject*> OuterObjects;
	OuterObjects.Reset();

	PropertyHandle->GetOuterObjects(OuterObjects);

	return OuterObjects.Num() ? OuterObjects[0] : nullptr;
}

class FImportDataSourceFileTransactionScope
{
public:
	FImportDataSourceFileTransactionScope(FText TransactionName, UObject* InOuterObject)
	{
		check(InOuterObject);
		OuterObject = InOuterObject;
		FScopedTransaction Transaction(TransactionName);

		bIsTransactionnal = (OuterObject->GetFlags() & RF_Transactional) > 0;
		if (!bIsTransactionnal)
		{
			OuterObject->SetFlags(RF_Transactional);
		}

		OuterObject->Modify();
	}

	~FImportDataSourceFileTransactionScope()
	{
		if (!bIsTransactionnal)
		{
			//Restore initial transactional flag value.
			OuterObject->ClearFlags(RF_Transactional);
		}
		OuterObject->MarkPackageDirty();
	}
private:
	bool bIsTransactionnal;
	UObject* OuterObject;
};

FReply FDatasmithImportInfoCustomization::OnBrowseSourceClicked() const
{
	SelectNewSource(GetCurrentScheme());

	return FReply::Handled();
}

bool FDatasmithImportInfoCustomization::SelectNewSource(FName SourceScheme) const
{
	UAssetImportData* ImportData = Cast<UAssetImportData>(GetOuterClass());
	UObject* Obj = ImportData ? ImportData->GetOuter() : nullptr;

	FDatasmithImportInfo* Info = GetImportInfo();
	if (!Obj || !Info)
	{
		return false;
	}

	FString SourceUri;
	FString FallbackFilepath;
	if (IDatasmithContentEditorModule::Get().BrowseExternalSourceUri(SourceScheme, Info->SourceUri, SourceUri, FallbackFilepath))
	{
		FImportDataSourceFileTransactionScope TransactionScope(LOCTEXT("SourceUriChanged", "Change Source URI"), ImportData);

		Info->SourceUri = SourceUri;
		Info->SourceHash.Empty();
		ImportData->UpdateFilenameOnly(FallbackFilepath);

		// Broadcasting property change to force refresh the asset registry tag and notify systems monitoring the URI.
		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(ImportData, EmptyPropertyChangedEvent);
		return true;
	}

	return false;
}

void FDatasmithImportInfoCustomization::OnSchemeComboBoxChanged(FName InItem, ESelectInfo::Type InSeletionInfo)
{
	if (!InItem.IsNone())
	{
		// User must select a new source when changing the scheme (type) of the source.
		SelectNewSource(InItem);

		// If we don't clear the selection OnSchemeComboBoxChanged() won't be triggered if the used select the same option again.
		SchemeComboBox->ClearSelection();
	}
}

#undef LOCTEXT_NAMESPACE
