// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SComboBox.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class UAssetImportData;
struct FDatasmithImportInfo;

class FDatasmithImportInfoCustomization : public IPropertyTypeCustomization
{
public:
	FDatasmithImportInfoCustomization();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	
	FName GetCurrentScheme() const;

	FReply OnBrowseSourceClicked() const;

	bool SelectNewSource(FName SourceScheme) const;

	FDatasmithImportInfo* GetImportInfo() const;

	/** Access the outer class that contains this struct */
	UObject* GetOuterClass() const;

	/** Get text for the UI */
	FText GetUriText() const;

	void OnSchemeComboBoxChanged(FName InItem, ESelectInfo::Type InSeletionInfo);

private:
	/** Property handle of the property we're editing */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	TSharedPtr<SComboBox<FName>> SchemeComboBox;

	TArray<FName> AvailableUriSchemes;
};