// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneActorDetailsPanel.h"

#include "DatasmithContentEditorModule.h"
#include "DatasmithContentModule.h"
#include "DatasmithScene.h"
#include "DatasmithSceneActor.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "UObject/UnrealType.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "IDetailsView.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "DatasmithSceneActor"


FDatasmithSceneActorDetailsPanel::FDatasmithSceneActorDetailsPanel()
	: bReimportDeletedActors(false)
{
}

TSharedRef<IDetailCustomization> FDatasmithSceneActorDetailsPanel::MakeInstance()
{
	return MakeShared< FDatasmithSceneActorDetailsPanel >();
}

void FDatasmithSceneActorDetailsPanel::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	SelectedObjectsList = DetailLayoutBuilder.GetSelectedObjects();

	TSharedRef<SWrapBox> WrapBox = SNew(SWrapBox).UseAllottedSize(true);

	FString CategoryName = TEXT("Datasmith");
	IDetailCategoryBuilder& ActionsCategory = DetailLayoutBuilder.EditCategory(*CategoryName);

	// Add the scene row first
	ActionsCategory.AddProperty( DetailLayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ADatasmithSceneActor, Scene ) ) );

	// Add the update actors button
	const FText ButtonCaption = LOCTEXT("UpdateActorsButton", "Update actors from Scene");
	const FText RespawnDeletedCheckBoxCaption = LOCTEXT("RespawnDeletedCheckbox", "Respawn deleted actors");

	auto IsChecked = [ this ]() -> ECheckBoxState { return bReimportDeletedActors ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; };
	auto RespawnDeletedCheckedStateChanged = [ this ]( ECheckBoxState NewState ) { bReimportDeletedActors = ( NewState == ECheckBoxState::Checked ); };

	const FText AutoReimportCaption = LOCTEXT("AutoReimportToggle", "Auto-Reimport");
	const FText AutoReimportTooltip = LOCTEXT("AutoReimportToogleTooltip", "Enable Auto-Reimport if the source associated with the DatasmithScene is an available DirectLink source.");

	WrapBox->AddSlot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(ButtonCaption)
				.OnClicked(	FOnClicked::CreateSP(this, &FDatasmithSceneActorDetailsPanel::OnExecuteAction) )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew(SCheckBox)
				.ToolTipText(RespawnDeletedCheckBoxCaption)
				.IsChecked_Lambda( IsChecked )
				.OnCheckStateChanged_Lambda( RespawnDeletedCheckedStateChanged )
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text( RespawnDeletedCheckBoxCaption )
			]
		];
	
	ActionsCategory.AddCustomRow(FText::GetEmpty())
		.ValueContent()
		[
			WrapBox
		];
}

FReply FDatasmithSceneActorDetailsPanel::OnExecuteAction()
{
	IDatasmithContentEditorModule& DatasmithContentEditorModule = FModuleManager::GetModuleChecked< IDatasmithContentEditorModule >( TEXT("DatasmithContentEditor") );

	for ( const TWeakObjectPtr< UObject >& SelectedObject : SelectedObjectsList )
	{
		ADatasmithSceneActor* SceneActor = Cast< ADatasmithSceneActor >( SelectedObject.Get() );
		DatasmithContentEditorModule.GetSpawnDatasmithSceneActorsHandler().ExecuteIfBound( SceneActor, bReimportDeletedActors );
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
