// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

enum class ECheckBoxState : uint8;
class IDetailLayoutBuilder;

class FDatasmithSceneActorDetailsPanel : public IDetailCustomization
{
public:
	FDatasmithSceneActorDetailsPanel();

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
	// End of IDetailCustomization interface

protected:
	FReply OnExecuteAction();

private:
	TArray< TWeakObjectPtr< UObject > > SelectedObjectsList;
	bool bReimportDeletedActors;
};
