// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSceneDetails.h"

#include "DatasmithAssetImportData.h"
#include "DatasmithContentEditorModule.h"
#include "DatasmithScene.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DatasmithSceneDetails"

void FDatasmithSceneDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized( Objects );
	check( Objects.Num() > 0 );
	UDatasmithScene* DatasmithScene = Cast< UDatasmithScene >( Objects[0].Get() );
	check( DatasmithScene );

	static_assert( std::is_same_v< typename TRemovePointer< typename TRemoveObjectPointer<decltype( UDatasmithScene::AssetImportData )>::Type >::Type, UDatasmithSceneImportData >, "Please update this details customization" );
	static_assert( TIsDerivedFrom < UDatasmithSceneImportData, UAssetImportData >::IsDerived, "Please update this details customization" );

	TSharedRef< IPropertyHandle > AssetImportDataPropertyHandle = DetailBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( UDatasmithScene, AssetImportData ) );
	TSharedPtr< IPropertyHandle > SourceDataHandle = AssetImportDataPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UAssetImportData, SourceData));
	if (SourceDataHandle)
	{
		// We don't want the default editable file selection UI, user should use the Source URI.
		SourceDataHandle->MarkHiddenByCustomization();
	}
}

void FDatasmithSceneDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CustomizeDetails( *DetailBuilder );
}

#undef LOCTEXT_NAMESPACE
