// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"

#define UE_API DATASMITHCONTENTEDITOR_API

class IDetailLayoutBuilder;

// Customization of the details of the Datasmith Scene for the data prep editor.
class FDatasmithSceneDetails : public IDetailCustomization
{
public:
	static TSharedRef< IDetailCustomization > MakeDetails() { return MakeShared<FDatasmithSceneDetails>(); };

	/** Called when details should be customized */
	UE_API virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	UE_API virtual void CustomizeDetails( const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder ) override;

private:

};

#undef UE_API
