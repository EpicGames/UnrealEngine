// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Developer/AssetTools/Public/AssetTypeCategories.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()
#include "Toolkits/IToolkit.h"

#define DATASMITHCONTENTEDITOR_MODULE_NAME TEXT("DatasmithContentEditor")

struct FGuid;
class UDatasmithImportOptions;
class UDatasmithScene;
class UObject;
class UPackage;
class UWorld;

// DATAPREP_TODO: Temporary interface to emulate future workflow. Interface to trigger build world and 'finalize' from data prep editor
class IDataprepImporterInterface
{
public:
	virtual ~IDataprepImporterInterface() = default;

	/**
	 * @param Namespace			The namespace in which to generate asset's unique ids.
	 * @param ImportWorld		The destination world that we will spawn the actors in.
	 * @param DatasmithScene	The DatasmithScene that we will apply the data prep pipeline on.
	 */
	virtual bool Initialize(const FString& Namespace, UWorld* ImportWorld, UDatasmithScene* DatasmithScene) = 0;
	virtual bool BuildWorld(TArray<TWeakObjectPtr<UObject>>& OutAssets) = 0;
	virtual bool SetFinalWorld(UWorld* FinalWorld) = 0;
	virtual bool FinalizeAssets(const TArray<TWeakObjectPtr<UObject>>& Assets) = 0;
	virtual TSubclassOf<class UDatasmithSceneImportData> GetAssetImportDataClass() const = 0;
};

DECLARE_DELEGATE_TwoParams( FOnSpawnDatasmithSceneActors, class ADatasmithSceneActor*, bool );
DECLARE_DELEGATE_ThreeParams( FOnCreateDatasmithSceneEditor, const EToolkitMode::Type, const TSharedPtr< class IToolkitHost >&, class UDatasmithScene*);
DECLARE_DELEGATE_RetVal(TSharedPtr<IDataprepImporterInterface>, FOnCreateDatasmithImportHandler );

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnSetAssetAutoReimport, UObject* /*Asset*/, bool /*bEnabled*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsAssetAutoReimportAvailable, UObject* /*Asset*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsAssetAutoReimportEnabled, UObject* /*Asset*/);
DECLARE_DELEGATE_RetVal_FourParams(bool, FOnBrowseExternalSourceUri, FName/*UriScheme*/, const FString& /*DefaultUri*/, FString& /*OutSourceUri*/, FString& /*OutFallbackFilepath*/);
DECLARE_DELEGATE_RetVal(const TArray<FName>&, FOnGetSupportedUriSchemes);

struct FImporterDescription
{
	FText Label;
	FText Description;
	FName StyleName;
	FString IconPath;
	TArray<FString> Formats;
	FString FilterString;
	FOnCreateDatasmithImportHandler Handler;
};


/**
 * The public interface of the DatasmithContent module
 */
class IDatasmithContentEditorModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to IDatasmithContentEditorModule
	 *
	 * @return Returns DatasmithContent singleton instance, loading the module on demand if needed
	 */
	static inline IDatasmithContentEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDatasmithContentEditorModule>(DATASMITHCONTENTEDITOR_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATASMITHCONTENTEDITOR_MODULE_NAME);
	}

	/**
	 * Delegate to spawn the actors related to a Datasmith scene. Called when the user triggers the action in the UI.
	 */
	virtual void RegisterSpawnDatasmithSceneActorsHandler( FOnSpawnDatasmithSceneActors SpawnActorsDelegate ) = 0;
	virtual void UnregisterSpawnDatasmithSceneActorsHandler( FOnSpawnDatasmithSceneActors SpawnActorsDelegate ) = 0;
	virtual FOnSpawnDatasmithSceneActors GetSpawnDatasmithSceneActorsHandler() const = 0;

	/**
	 * Delegate to creation of the datasmith scene editor. The action is in this module while the datasmith scene editor is in its own plugin
	 */
	virtual void RegisterDatasmithSceneEditorHandler(FOnCreateDatasmithSceneEditor InCreateDatasmithSceneEditor) = 0;
	virtual void UnregisterDatasmithSceneEditorHandler(FOnCreateDatasmithSceneEditor InCreateDatasmithSceneEditor) = 0;
	virtual FOnCreateDatasmithSceneEditor GetDatasmithSceneEditorHandler() const = 0;

	/**
	* Delegate to creation of the datasmith scene editor. The action is in this module while the datasmith scene editor is in its own plugin
	*/
	virtual void RegisterDatasmithImporter(const void* Registrar, const FImporterDescription& ImporterDescription) = 0;
	virtual void UnregisterDatasmithImporter(const void* Registrar) = 0;
	virtual TArray<FImporterDescription> GetDatasmithImporters() = 0;

	/**
	 * Delegate to enable or disable AutoReimport of an asset.
	 */
	virtual void RegisterSetAssetAutoReimportHandler(FOnSetAssetAutoReimport&& SetAssetAutoReimportDelegate) = 0;
	virtual void UnregisterSetAssetAutoReimportHandler(FDelegateHandle InHandle) = 0;
	virtual TOptional<bool> SetAssetAutoReimport(UObject* Asset, bool bEnabled) const = 0;

	/**
	 * Delegate returning if the AutoReimport feature is available for a given asset.
	 */
	virtual void RegisterIsAssetAutoReimportAvailableHandler(FOnIsAssetAutoReimportAvailable&& IsAssetAutoReimportAvailableDelegate) = 0;
	virtual void UnregisterIsAssetAutoReimportAvailableHandler(FDelegateHandle InHandle) = 0;
	virtual TOptional<bool> IsAssetAutoReimportAvailable(UObject* Asset) const = 0;

	/**
	 * Delegate returning if the AutoReimport is currently active for a given asset.
	 */
	virtual void RegisterIsAssetAutoReimportEnabledHandler(FOnIsAssetAutoReimportEnabled&& IsAssetAutoReimportEnabledDelegate) = 0;
	virtual void UnregisterIsAssetAutoReimportEnabledHandler(FDelegateHandle InHandle) = 0;
	virtual TOptional<bool> IsAssetAutoReimportEnabled(UObject* Asset) const = 0;

	virtual void RegisterBrowseExternalSourceUriHandler(FOnBrowseExternalSourceUri&& BrowseExternalSourceUriDelegate) = 0;
	virtual void UnregisterBrowseExternalSourceUriHandler(FDelegateHandle InHandle) = 0;
	virtual bool BrowseExternalSourceUri(FName UriScheme, const FString& DefaultUri, FString& OutSourceUri, FString& OutFallbackFilepath) const = 0;

	virtual void RegisterGetSupportedUriSchemeHandler(FOnGetSupportedUriSchemes&& GetSupportedUriSchemeDelegate) = 0;
	virtual void UnregisterGetSupportedUriSchemeHandler(FDelegateHandle InHandle) = 0;
	virtual TOptional<TArray<FName>> GetSupportedUriScheme() const = 0;
};

