// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

#define UE_API DATASMITHCONTENTEDITOR_API

/** Contains a collection of named properties (StyleSet) that guide the appearance of Datasmith related UI. */
class FDatasmithContentEditorStyle
{
public:
	static UE_API void Initialize();
	static UE_API void Shutdown();
	static TSharedPtr<ISlateStyle> Get() { return StyleSet; }

	static UE_API FName GetStyleSetName();

private:
	static UE_API FString InContent(const FString& RelativePath, const TCHAR* Extension);

private:
	static UE_API TSharedPtr<FSlateStyleSet> StyleSet;
};

#undef UE_API
